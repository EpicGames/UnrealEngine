{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Content Browser - Class Data Source",
	"Description" : "Data Source plugin providing Class Data to the Content Browser",
	"Category" : "Content Browser",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "ContentBrowserClassDataSource",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	]
}