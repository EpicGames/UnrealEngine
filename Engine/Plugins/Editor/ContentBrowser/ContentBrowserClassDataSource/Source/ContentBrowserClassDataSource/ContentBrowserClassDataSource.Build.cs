// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ContentBrowserClassDataSource : ModuleRules
	{
		public ContentBrowserClassDataSource(ReadOnlyTargetRules Target) : base(Target)
		{
			ShortName = "CBClassDataSource";

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"ContentBrowserData",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetTools",
					"CollectionManager",
					"UnrealEd",
					"Projects",
					"GameProjectGeneration",
					"ToolMenus",
					"Slate",
					"SlateCore",
				}
			);
		}
	}
}
