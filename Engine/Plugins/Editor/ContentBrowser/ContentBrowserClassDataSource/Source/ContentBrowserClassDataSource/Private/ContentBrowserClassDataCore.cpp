// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserClassDataCore.h"
#include "ContentBrowserClassDataPayload.h"
#include "ContentBrowserDataSource.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/PackageName.h"
#include "AssetViewUtils.h"
#include "IAssetTypeActions.h"
#include "ContentBrowserDataUtils.h"
#include "UObject/AssetRegistryTagsContext.h"

#define LOCTEXT_NAMESPACE "ContentBrowserClassDataSource"

namespace ContentBrowserClassData
{

bool GetUnrealContentRootFromInternalClassPath(const FName InPath, FString& OutUnrealContentRoot)
{
	const FStringView ClassesRootPath = TEXT("/Classes_");

	// Internal class paths are all expected to start with "/Classes_" 
	// where the component after the underscore is the Unreal content root
	FNameBuilder PathStr(InPath);
	const FStringView PathStrView = PathStr;
	if (!PathStrView.StartsWith(ClassesRootPath))
	{
		return false;
	}

	// The module name ends at the first slash after the root (if any)
	FStringView UnrealContentRootStr = PathStrView.Mid(ClassesRootPath.Len());
	{
		int32 LastSlashIndex = INDEX_NONE;
		if (UnrealContentRootStr.FindChar(TEXT('/'), LastSlashIndex))
		{
			UnrealContentRootStr = UnrealContentRootStr.Left(LastSlashIndex);
		}
	}

	OutUnrealContentRoot = UnrealContentRootStr;
	return true;
}

bool IsEngineClass(const FName InPath)
{
	FString PathStr;
	if (!GetUnrealContentRootFromInternalClassPath(InPath, PathStr))
	{
		return false;
	}
	PathStr.InsertAt(0, TEXT('/'));

	if (AssetViewUtils::IsEngineFolder(PathStr))
	{
		return true;
	}

	EPluginLoadedFrom PluginSource = EPluginLoadedFrom::Engine;
	if (AssetViewUtils::IsPluginFolder(PathStr, &PluginSource))
	{
		return PluginSource == EPluginLoadedFrom::Engine;
	}

	return false;
}

bool IsProjectClass(const FName InPath)
{
	FString PathStr;
	if (!GetUnrealContentRootFromInternalClassPath(InPath, PathStr))
	{
		return false;
	}
	PathStr.InsertAt(0, TEXT('/'));

	if (AssetViewUtils::IsProjectFolder(PathStr))
	{
		return true;
	}

	EPluginLoadedFrom PluginSource = EPluginLoadedFrom::Engine;
	if (AssetViewUtils::IsPluginFolder(PathStr, &PluginSource))
	{
		return PluginSource == EPluginLoadedFrom::Project;
	}

	return false;
}

bool IsPluginClass(const FName InPath)
{
	FString PathStr;
	if (!GetUnrealContentRootFromInternalClassPath(InPath, PathStr))
	{
		return false;
	}
	PathStr.InsertAt(0, TEXT('/'));

	return AssetViewUtils::IsPluginFolder(PathStr);
}

FContentBrowserItemData CreateClassFolderItem(
	UContentBrowserDataSource* InOwnerDataSource, const FName InVirtualPath, FName InFolderPath, const bool bIsFromPlugin)
{
	static const FName GameRootPath = "/Classes_Game";
	static const FName EngineRootPath = "/Classes_Engine";

	const FString FolderItemName = FPackageName::GetShortName(InFolderPath);

	FText FolderDisplayNameOverride;
	if (InFolderPath == GameRootPath)
	{
		FolderDisplayNameOverride = LOCTEXT("GameFolderDisplayName", "C++ Classes");
	}
	else if (InFolderPath == EngineRootPath)
	{
		FolderDisplayNameOverride = LOCTEXT("EngineFolderDisplayName", "Engine C++ Classes");
	}
	else
	{
		FolderDisplayNameOverride = ContentBrowserDataUtils::GetFolderItemDisplayNameOverride(InFolderPath, FolderItemName, /*bIsClassesFolder*/ true);
	}

	return FContentBrowserItemData(InOwnerDataSource,
		EContentBrowserItemFlags::Type_Folder | EContentBrowserItemFlags::Category_Class | (bIsFromPlugin ? EContentBrowserItemFlags::Category_Plugin : EContentBrowserItemFlags::None),
		InVirtualPath,
		*FolderItemName,
		MoveTemp(FolderDisplayNameOverride),
		MakeShared<FContentBrowserClassFolderItemDataPayload>(InFolderPath),
		{ InFolderPath });
}

FContentBrowserItemData CreateClassFileItem(
	UContentBrowserDataSource* InOwnerDataSource,
	const FName InVirtualPath,
	FName InClassPath,
	UClass* InClass,
	const bool bIsFromPlugin)
{
	return FContentBrowserItemData(InOwnerDataSource,
		EContentBrowserItemFlags::Type_File | EContentBrowserItemFlags::Category_Class | (bIsFromPlugin ? EContentBrowserItemFlags::Category_Plugin : EContentBrowserItemFlags::None),
		InVirtualPath,
		InClass->GetFName(),
		FText(),
		MakeShared<FContentBrowserClassFileItemDataPayload>(InClassPath, InClass),
		{ InClassPath });
}

TSharedPtr<const FContentBrowserClassFolderItemDataPayload> GetClassFolderItemPayload(
	const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem)
{
	if (InItem.GetOwnerDataSource() == InOwnerDataSource && InItem.IsFolder())
	{
		return StaticCastSharedPtr<const FContentBrowserClassFolderItemDataPayload>(InItem.GetPayload());
	}
	return nullptr;
}

TSharedPtr<const FContentBrowserClassFileItemDataPayload> GetClassFileItemPayload(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem)
{
	if (InItem.GetOwnerDataSource() == InOwnerDataSource && InItem.IsFile())
	{
		return StaticCastSharedPtr<const FContentBrowserClassFileItemDataPayload>(InItem.GetPayload());
	}
	return nullptr;
}

void EnumerateClassFolderItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserClassFolderItemDataPayload>&)> InFolderPayloadCallback)
{
	for (const FContentBrowserItemData& Item : InItems)
	{
		if (TSharedPtr<const FContentBrowserClassFolderItemDataPayload> FolderPayload = GetClassFolderItemPayload(InOwnerDataSource, Item))
		{
			if (!InFolderPayloadCallback(FolderPayload.ToSharedRef()))
			{
				break;
			}
		}
	}
}

void EnumerateClassFileItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserClassFileItemDataPayload>&)> InClassPayloadCallback)
{
	for (const FContentBrowserItemData& Item : InItems)
	{
		if (TSharedPtr<const FContentBrowserClassFileItemDataPayload> ClassPayload = GetClassFileItemPayload(InOwnerDataSource, Item))
		{
			if (!InClassPayloadCallback(ClassPayload.ToSharedRef()))
			{
				break;
			}
		}
	}
}

void EnumerateClassItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserClassFolderItemDataPayload>&)> InFolderPayloadCallback, TFunctionRef<bool(const TSharedRef<const FContentBrowserClassFileItemDataPayload>&)> InClassPayloadCallback)
{
	for (const FContentBrowserItemData& Item : InItems)
	{
		if (TSharedPtr<const FContentBrowserClassFolderItemDataPayload> FolderPayload = GetClassFolderItemPayload(InOwnerDataSource, Item))
		{
			if (!InFolderPayloadCallback(FolderPayload.ToSharedRef()))
			{
				break;
			}
		}

		if (TSharedPtr<const FContentBrowserClassFileItemDataPayload> ClassPayload = GetClassFileItemPayload(InOwnerDataSource, Item))
		{
			if (!InClassPayloadCallback(ClassPayload.ToSharedRef()))
			{
				break;
			}
		}
	}
}

void SetOptionalErrorMessage(FText* OutErrorMsg, FText InErrorMsg)
{
	if (OutErrorMsg)
	{
		*OutErrorMsg = MoveTemp(InErrorMsg);
	}
}

bool CanEditItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	if (TSharedPtr<const FContentBrowserClassFileItemDataPayload> ClassPayload = GetClassFileItemPayload(InOwnerDataSource, InItem))
	{
		return CanEditClassFileItem(*ClassPayload, OutErrorMsg);
	}

	return false;
}

bool CanEditClassFileItem(const FContentBrowserClassFileItemDataPayload& InClassPayload, FText* OutErrorMsg)
{
	return true;
}

bool EditItems(IAssetTypeActions* InClassTypeActions, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems)
{
	TArray<TSharedRef<const FContentBrowserClassFileItemDataPayload>, TInlineAllocator<16>> ClassPayloads;

	EnumerateClassFileItemPayloads(InOwnerDataSource, InItems, [&ClassPayloads](const TSharedRef<const FContentBrowserClassFileItemDataPayload>& InClassPayload)
	{
		ClassPayloads.Add(InClassPayload);
		return true;
	});

	return EditClassFileItems(InClassTypeActions, ClassPayloads);
}

bool EditClassFileItems(IAssetTypeActions* InClassTypeActions, TArrayView<const TSharedRef<const FContentBrowserClassFileItemDataPayload>> InClassPayloads)
{
	TArray<UObject*> ClassList;
	for (const TSharedRef<const FContentBrowserClassFileItemDataPayload>& ClassPayload : InClassPayloads)
	{
		if (UClass* ClassPtr = ClassPayload->GetClass())
		{
			ClassList.Add(ClassPtr);
		}
	}

	if (ClassList.Num() > 0)
	{
		InClassTypeActions->OpenAssetEditor(ClassList);
		return true;
	}

	return false;
}

bool UpdateItemThumbnail(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FAssetThumbnail& InThumbnail)
{
	if (TSharedPtr<const FContentBrowserClassFileItemDataPayload> ClassPayload = GetClassFileItemPayload(InOwnerDataSource, InItem))
	{
		return UpdateClassFileItemThumbnail(*ClassPayload, InThumbnail);
	}

	return false;
}

bool UpdateClassFileItemThumbnail(const FContentBrowserClassFileItemDataPayload& InClassPayload, FAssetThumbnail& InThumbnail)
{
	InClassPayload.UpdateThumbnail(InThumbnail);
	return true;
}

bool AppendItemReference(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& InOutStr)
{
	if (TSharedPtr<const FContentBrowserClassFileItemDataPayload> ClassPayload = GetClassFileItemPayload(InOwnerDataSource, InItem))
	{
		return AppendClassFileItemReference(*ClassPayload, InOutStr);
	}

	return false;
}

bool AppendItemObjectPath(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& InOutStr)
{
	if (TSharedPtr<const FContentBrowserClassFileItemDataPayload> ClassPayload = GetClassFileItemPayload(InOwnerDataSource, InItem))
	{
		return AppendObjectPathFileItemReference(*ClassPayload, InOutStr);
	}

	return false;
}

bool AppendItemPackageName(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& InOutStr)
{
	if (TSharedPtr<const FContentBrowserClassFileItemDataPayload> ClassPayload = GetClassFileItemPayload(InOwnerDataSource, InItem))
	{
		return AppendPackageNameItemReference(*ClassPayload, InOutStr);
	}

	return false;
}

bool AppendClassFileItemReference(const FContentBrowserClassFileItemDataPayload& InClassPayload, FString& InOutStr)
{
	if (InOutStr.Len() > 0)
	{
		InOutStr += LINE_TERMINATOR;
	}
	InOutStr += InClassPayload.GetAssetData().GetExportTextName();
	return true;
}

bool AppendObjectPathFileItemReference(const FContentBrowserClassFileItemDataPayload& InClassPayload, FString& InOutStr)
{
	if (InOutStr.Len() > 0)
	{
		InOutStr += LINE_TERMINATOR;
	}
	InClassPayload.GetAssetData().AppendObjectPath(InOutStr);
	return true;
}

bool AppendPackageNameItemReference(const FContentBrowserClassFileItemDataPayload& InClassPayload, FString& InOutStr)
{
	if (InOutStr.Len() > 0)
	{
		InOutStr += LINE_TERMINATOR;
	}
	InClassPayload.GetAssetData().PackageName.AppendString(InOutStr);
	return true;
}

bool GetItemPhysicalPath(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& OutDiskPath)
{
	if (TSharedPtr<const FContentBrowserClassFolderItemDataPayload> FolderPayload = GetClassFolderItemPayload(InOwnerDataSource, InItem))
	{
		return GetClassFolderItemPhysicalPath(*FolderPayload, OutDiskPath);
	}

	if (TSharedPtr<const FContentBrowserClassFileItemDataPayload> ClassPayload = GetClassFileItemPayload(InOwnerDataSource, InItem))
	{
		return GetClassFileItemPhysicalPath(*ClassPayload, OutDiskPath);
	}

	return false;
}

bool GetClassFolderItemPhysicalPath(const FContentBrowserClassFolderItemDataPayload& InFolderPayload, FString& OutDiskPath)
{
	const FString& FolderFilename = InFolderPayload.GetFilename();
	if (!FolderFilename.IsEmpty())
	{
		OutDiskPath = FolderFilename;
		return true;
	}

	return false;
}

bool GetClassFileItemPhysicalPath(const FContentBrowserClassFileItemDataPayload& InClassPayload, FString& OutDiskPath)
{
	const FString& ClassFilename = InClassPayload.GetFilename();
	if (!ClassFilename.IsEmpty())
	{
		OutDiskPath = ClassFilename;
		return true;
	}

	return false;
}

struct FClassTagDefinition
{
	/** The kind of data represented by this tag value */
	UObject::FAssetRegistryTag::ETagType TagType = UObject::FAssetRegistryTag::TT_Hidden;

	/** Flags giving hints at how to display this tag value in the UI (see ETagDisplay) */
	uint32 DisplayFlags = UObject::FAssetRegistryTag::TD_None;

	/** Resolved display name of the associated tag */
	FText DisplayName;
};

using FClassTagDefinitionMap = TSortedMap<FName, FClassTagDefinition, FDefaultAllocator, FNameFastLess>;

const FClassTagDefinitionMap& GetAvailableClassTags()
{
	static const FClassTagDefinitionMap ClassTags = []()
	{
		FClassTagDefinitionMap ClassTagsTmp;

		const UObject* ClassDefault = GetDefault<UClass>();
		FAssetRegistryTagsContextData TagsContext(ClassDefault, EAssetRegistryTagsCaller::Uncategorized);
		ClassDefault->GetAssetRegistryTags(TagsContext);

		for (const TPair<FName,UObject::FAssetRegistryTag>& TagPair : TagsContext.Tags)
		{
			FClassTagDefinition& ClassTag = ClassTagsTmp.Add(TagPair.Key);
			ClassTag.TagType = TagPair.Value.Type;
			ClassTag.DisplayFlags = TagPair.Value.DisplayFlags;
			ClassTag.DisplayName = FText::AsCultureInvariant(FName::NameToDisplayString(TagPair.Key.ToString(), /*bIsBool*/false));
		}

		return ClassTagsTmp;
	}();

	return ClassTags;
}

void GetClassItemAttribute(const bool InIncludeMetaData, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	OutAttributeValue.SetValue(NAME_Class);

	if (InIncludeMetaData)
	{
		static const FText ClassDisplayName = LOCTEXT("AttributeDisplayName_Class", "Class");

		FContentBrowserItemDataAttributeMetaData AttributeMetaData;
		AttributeMetaData.AttributeType = UObject::FAssetRegistryTag::TT_Hidden;
		AttributeMetaData.DisplayName = ClassDisplayName;
		OutAttributeValue.SetMetaData(MoveTemp(AttributeMetaData));
	}
}

void GetGenericItemAttribute(const FName InTagKey, const FString& InTagValue, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	check(!InTagKey.IsNone());

	if (FTextStringHelper::IsComplexText(*InTagValue))
	{
		FText TmpText;
		if (FTextStringHelper::ReadFromBuffer(*InTagValue, TmpText))
		{
			OutAttributeValue.SetValue(MoveTemp(TmpText));
		}
	}
	if (!OutAttributeValue.IsValid())
	{
		OutAttributeValue.SetValue(InTagValue);
	}

	if (InIncludeMetaData)
	{
		FContentBrowserItemDataAttributeMetaData AttributeMetaData;
		if (const FClassTagDefinition* ClassTagCache = GetAvailableClassTags().Find(InTagKey))
		{
			AttributeMetaData.AttributeType = ClassTagCache->TagType;
			AttributeMetaData.DisplayFlags = ClassTagCache->DisplayFlags;
			AttributeMetaData.DisplayName = ClassTagCache->DisplayName;
		}
		else
		{
			AttributeMetaData.DisplayName = FText::AsCultureInvariant(FName::NameToDisplayString(InTagKey.ToString(), /*bIsBool*/false));
		}
		OutAttributeValue.SetMetaData(MoveTemp(AttributeMetaData));
	}
}

bool GetItemAttribute(IAssetTypeActions* InClassTypeActions, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	if (TSharedPtr<const FContentBrowserClassFolderItemDataPayload> FolderPayload = GetClassFolderItemPayload(InOwnerDataSource, InItem))
	{
		return GetClassFolderItemAttribute(*FolderPayload, InIncludeMetaData, InAttributeKey, OutAttributeValue);
	}

	if (TSharedPtr<const FContentBrowserClassFileItemDataPayload> ClassPayload = GetClassFileItemPayload(InOwnerDataSource, InItem))
	{
		return GetClassFileItemAttribute(InClassTypeActions, *ClassPayload, InIncludeMetaData, InAttributeKey, OutAttributeValue);
	}

	return false;
}

bool GetClassFolderItemAttribute(const FContentBrowserClassFolderItemDataPayload& InFolderPayload, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	// Hard-coded attribute keys
	{
		if (InAttributeKey == ContentBrowserItemAttributes::ItemIsEngineContent)
		{
			const bool bIsEngineFolder = IsEngineClass(InFolderPayload.GetInternalPath());
			OutAttributeValue.SetValue(bIsEngineFolder);
			return true;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemIsProjectContent)
		{
			const bool bIsProjectFolder = IsProjectClass(InFolderPayload.GetInternalPath());
			OutAttributeValue.SetValue(bIsProjectFolder);
			return true;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemIsPluginContent)
		{
			const bool bIsPluginFolder = IsPluginClass(InFolderPayload.GetInternalPath());
			OutAttributeValue.SetValue(bIsPluginFolder);
			return true;
		}
	}

	return false;
}

bool GetClassFileItemAttribute(IAssetTypeActions* InClassTypeActions, const FContentBrowserClassFileItemDataPayload& InClassPayload, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	// Hard-coded attribute keys
	{
		static const FName NAME_Type = "Type";

		if (InAttributeKey == ContentBrowserItemAttributes::ItemTypeName || InAttributeKey == NAME_Class || InAttributeKey == NAME_Type)
		{
			GetClassItemAttribute(InIncludeMetaData, OutAttributeValue);
			return true;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemTypeDisplayName)
		{
			const FText AssetDisplayName = InClassTypeActions->GetDisplayNameFromAssetData(InClassPayload.GetAssetData());
			if (!AssetDisplayName.IsEmpty())
			{
				OutAttributeValue.SetValue(AssetDisplayName);
			}
			else
			{
				OutAttributeValue.SetValue(InClassTypeActions->GetName());
			}

			return true;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemDescription)
		{
			const FText AssetDescription = InClassTypeActions->GetAssetDescription(InClassPayload.GetAssetData());
			if (!AssetDescription.IsEmpty())
			{
				OutAttributeValue.SetValue(AssetDescription);
				return true;
			}
			return false;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemIsEngineContent)
		{
			const bool bIsEngineFolder = IsEngineClass(InClassPayload.GetInternalPath());
			OutAttributeValue.SetValue(bIsEngineFolder);
			return true;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemIsProjectContent)
		{
			const bool bIsProjectFolder = IsProjectClass(InClassPayload.GetInternalPath());
			OutAttributeValue.SetValue(bIsProjectFolder);
			return true;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemIsPluginContent)
		{
			const bool bIsPluginFolder = IsPluginClass(InClassPayload.GetInternalPath());
			OutAttributeValue.SetValue(bIsPluginFolder);
			return true;
		}

		if (InAttributeKey == ContentBrowserItemAttributes::ItemColor)
		{
			const FLinearColor AssetColor = InClassTypeActions->GetTypeColor();//.ReinterpretAsLinear();
			OutAttributeValue.SetValue(AssetColor.ToString());
			return true;
		}
	}

	// Generic attribute keys
	{
		const FAssetData& AssetData = InClassPayload.GetAssetData();

		FName FoundAttributeKey = InAttributeKey;
		FAssetDataTagMapSharedView::FFindTagResult FoundValue = AssetData.TagsAndValues.FindTag(FoundAttributeKey);
		if (FoundValue.IsSet())
		{
			GetGenericItemAttribute(FoundAttributeKey, FoundValue.GetValue(), InIncludeMetaData, OutAttributeValue);
			return true;
		}
	}

	return false;
}

bool GetItemAttributes(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues)
{
	if (TSharedPtr<const FContentBrowserClassFileItemDataPayload> ClassPayload = GetClassFileItemPayload(InOwnerDataSource, InItem))
	{
		return GetClassFileItemAttributes(*ClassPayload, InIncludeMetaData, OutAttributeValues);
	}

	return false;
}

bool GetClassFileItemAttributes(const FContentBrowserClassFileItemDataPayload& InClassPayload, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues)
{
	// Hard-coded attribute keys
	{
		FContentBrowserItemDataAttributeValue& ClassAttributeValue = OutAttributeValues.Add(NAME_Class);
		GetClassItemAttribute(InIncludeMetaData, ClassAttributeValue);
	}

	// Generic attribute keys
	{
		const FAssetData& AssetData = InClassPayload.GetAssetData();

		OutAttributeValues.Reserve(OutAttributeValues.Num() + AssetData.TagsAndValues.Num());
		for (const auto& TagAndValue : AssetData.TagsAndValues)
		{
			FContentBrowserItemDataAttributeValue& GenericAttributeValue = OutAttributeValues.Add(TagAndValue.Key);
			GetGenericItemAttribute(TagAndValue.Key, TagAndValue.Value.AsString(), InIncludeMetaData, GenericAttributeValue);
		}
	}

	return true;
}

}

#undef LOCTEXT_NAMESPACE
