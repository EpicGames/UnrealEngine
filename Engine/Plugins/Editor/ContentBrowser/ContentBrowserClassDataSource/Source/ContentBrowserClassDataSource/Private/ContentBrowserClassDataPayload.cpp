// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserClassDataPayload.h"
#include "Misc/Paths.h"
#include "AssetThumbnail.h"
#include "SourceCodeNavigation.h"

const FString& FContentBrowserClassFolderItemDataPayload::GetFilename() const
{
	if (!bHasCachedFilename)
	{
		// Split the class path into its component parts
		TArray<FString> ClassPathParts;
		InternalPath.ToString().ParseIntoArray(ClassPathParts, TEXT("/"), true);

		// We need to have at least two sections (a root, and a module name) to be able to resolve a file system path
		if (ClassPathParts.Num() >= 2)
		{
			// Get the base file path to the module, and then append any remaining parts of the class path (as the remaining parts mirror the file system)
			if (FSourceCodeNavigation::FindModulePath(ClassPathParts[1], CachedFilename))
			{
				for (int32 PathPartIndex = 2; PathPartIndex < ClassPathParts.Num(); ++PathPartIndex)
				{
					CachedFilename /= ClassPathParts[PathPartIndex];
				}

				CachedFilename = FPaths::ConvertRelativePathToFull(CachedFilename);
			}
		}

		bHasCachedFilename = true;
	}
	return CachedFilename;
}


FContentBrowserClassFileItemDataPayload::FContentBrowserClassFileItemDataPayload(const FName InInternalPath, UClass* InClass)
	: InternalPath(InInternalPath)
	, Class(InClass)
	, AssetData(InClass)
{	
}

const FString& FContentBrowserClassFileItemDataPayload::GetFilename() const
{
	if (!bHasCachedFilename)
	{
		const FString PackageNameStr = AssetData.PackageName.ToString();

		static const FStringView ScriptString = TEXT("/Script/");
		if (FStringView(PackageNameStr).StartsWith(ScriptString))
		{
			// Handle C++ classes specially, as FPackageName::LongPackageNameToFilename won't return the correct path in this case
			const FString ModuleName = PackageNameStr.RightChop(ScriptString.Len());
			FString ModulePath;
			if (FSourceCodeNavigation::FindModulePath(ModuleName, ModulePath))
			{
				FString RelativePath;
				if (AssetData.GetTagValue("ModuleRelativePath", RelativePath))
				{
					CachedFilename = FPaths::ConvertRelativePathToFull(ModulePath / RelativePath);
				}
			}
		}

		bHasCachedFilename = true;
	}

	return CachedFilename;
}

void FContentBrowserClassFileItemDataPayload::UpdateThumbnail(FAssetThumbnail& InThumbnail) const
{
	InThumbnail.SetAsset(AssetData);
}
