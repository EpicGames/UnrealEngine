// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserClassDataSource.h"
#include "ContentBrowserClassDataCore.h"
#include "AssetToolsModule.h"
#include "ContentBrowserClassDataPayload.h"
#include "ICollectionContainer.h"
#include "ContentBrowserDataFilter.h"
#include "ToolMenus.h"
#include "ContentBrowserDataMenuContexts.h"
#include "NewClassContextMenu.h"
#include "GameProjectGenerationModule.h"
#include "Framework/Docking/TabManager.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserDataUtils.h"
#include "ContentBrowserItemPath.h"
#include "Editor/UnrealEdEngine.h"
#include "IAssetTools.h"
#include "Preferences/UnrealEdOptions.h"
#include "IAssetTypeActions.h"
#include "UnrealEdGlobals.h"
#include "ToolMenu.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBrowserClassDataSource)

#define LOCTEXT_NAMESPACE "ContentBrowserClassDataSource"

void UContentBrowserClassDataSource::Initialize(const bool InAutoRegister)
{
	Super::Initialize(InAutoRegister);

	// Bind the class specific menu extensions
	{
		if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AddNewContextMenu"))
		{
			Menu->AddDynamicSection(*FString::Printf(TEXT("DynamicSection_DataSource_%s"), *GetName()), FNewToolMenuDelegate::CreateLambda([WeakThis = TWeakObjectPtr<UContentBrowserClassDataSource>(this)](UToolMenu* InMenu)
			{
				if (UContentBrowserClassDataSource* This = WeakThis.Get())
				{
					This->PopulateAddNewContextMenu(InMenu);
				}
			}));
		}
	}

	BuildRootPathVirtualTree();
}

void UContentBrowserClassDataSource::Shutdown()
{
	NativeClassHierarchy.Reset();

	Super::Shutdown();
}

TSharedPtr<IAssetTypeActions> UContentBrowserClassDataSource::GetClassTypeActions()
{
	if (!ClassTypeActions)
	{
		static const FName NAME_AssetTools = "AssetTools";
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(NAME_AssetTools);
		ClassTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(UClass::StaticClass()).Pin();
	}

	return ClassTypeActions;
}

void UContentBrowserClassDataSource::BuildRootPathVirtualTree()
{
	Super::BuildRootPathVirtualTree();

	ConditionalCreateNativeClassHierarchy();

	TArray<FName> InternalRoots;
	NativeClassHierarchy->GetClassRoots(InternalRoots, true, true);

	for (const FName InternalRoot : InternalRoots)
	{
		RootPathAdded(FNameBuilder(InternalRoot));
	}
}

bool UContentBrowserClassDataSource::RootClassPathPassesFilter(const FName InRootClassPath, const bool bIncludeEngineClasses, const bool bIncludePluginClasses) const
{
	// Remove "/" prefix
	FNameBuilder RootNodeString(InRootClassPath);
	FStringView RootNodeNameView(RootNodeString);
	RootNodeNameView.RightChopInline(1);

	return NativeClassHierarchy->RootNodePassesFilter(FName(RootNodeNameView), bIncludeEngineClasses, bIncludePluginClasses);
}

void UContentBrowserClassDataSource::CompileFilter(const FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserClassDataSource::CompileFilter);

	const FContentBrowserDataClassFilter* ClassFilter = InFilter.ExtraFilters.FindFilter<FContentBrowserDataClassFilter>();
	const FContentBrowserDataCollectionFilter* CollectionFilter = InFilter.ExtraFilters.FindFilter<FContentBrowserDataCollectionFilter>();

	const FPathPermissionList* ClassPermissionList = ClassFilter && ClassFilter->ClassPermissionList && ClassFilter->ClassPermissionList->HasFiltering() ? ClassFilter->ClassPermissionList.Get() : nullptr;

	const bool bIncludeFolders = EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders);
	const bool bIncludeFiles = EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles);

	const bool bIncludeClasses = EnumHasAnyFlags(InFilter.ItemCategoryFilter, EContentBrowserItemCategoryFilter::IncludeClasses);

	FContentBrowserDataFilterList& FilterList = OutCompiledFilter.CompiledFilters.FindOrAdd(this);
	FContentBrowserCompiledClassDataFilter& ClassDataFilter = FilterList.FindOrAddFilter<FContentBrowserCompiledClassDataFilter>();

	// If we aren't including anything, then we can just bail now
	if (!bIncludeClasses || (!bIncludeFolders && !bIncludeFiles))
	{
		return;
	}

	ConditionalCreateNativeClassHierarchy();
	RefreshVirtualPathTreeIfNeeded();

	FName ConvertedPath;
	const EContentBrowserPathType ConvertedPathType = TryConvertVirtualPath(InPath, ConvertedPath);

	TSet<FName> InternalPaths;
	if (ConvertedPathType == EContentBrowserPathType::Internal)
	{
		InternalPaths.Add(ConvertedPath);
	}
	else if (ConvertedPathType != EContentBrowserPathType::Virtual)
	{
		return;
	}

	const bool bIncludeEngine = EnumHasAnyFlags(InFilter.ItemAttributeFilter, EContentBrowserItemAttributeFilter::IncludeEngine);
	const bool bIncludePlugins = EnumHasAnyFlags(InFilter.ItemAttributeFilter, EContentBrowserItemAttributeFilter::IncludePlugins);

	if (bIncludeFolders)
	{
		if (InFilter.bRecursivePaths)
		{
			if (ConvertedPathType == EContentBrowserPathType::Virtual)
			{
				RootPathVirtualTree.EnumerateSubPaths(InPath, [this, &InternalPaths, bIncludeEngine, bIncludePlugins](FName VirtualSubPath, FName InternalSubPath)
				{
					if (!InternalSubPath.IsNone())
					{
						if (RootClassPathPassesFilter(InternalSubPath, bIncludeEngine, bIncludePlugins))
						{
							InternalPaths.Add(InternalSubPath);
						}
					}
					return true;
				}, true);
			}
		}
		else
		{
			if (ConvertedPathType == EContentBrowserPathType::Virtual)
			{
				FContentBrowserCompiledVirtualFolderFilter* VirtualFolderFilter = nullptr;
				RootPathVirtualTree.EnumerateSubPaths(InPath, [this, &InternalPaths, &VirtualFolderFilter, &FilterList, bIncludeEngine, bIncludePlugins](FName VirtualSubPath, FName InternalSubPath)
				{
					if (!InternalSubPath.IsNone())
					{
						if (RootClassPathPassesFilter(InternalSubPath, bIncludeEngine, bIncludePlugins))
						{
							InternalPaths.Add(InternalSubPath);
						}
					}
					else
					{
						// Determine if any internal path under VirtualSubPath passes
						bool bPassesFilter = false;
						RootPathVirtualTree.EnumerateSubPaths(VirtualSubPath, [this, &VirtualFolderFilter, &FilterList, &bPassesFilter, bIncludeEngine, bIncludePlugins](FName RecursiveVirtualSubPath, FName RecursiveInternalSubPath)
						{
							bPassesFilter = bPassesFilter || (!RecursiveInternalSubPath.IsNone() && RootClassPathPassesFilter(RecursiveInternalSubPath, bIncludeEngine, bIncludePlugins));
							return bPassesFilter == false;
						}, true);

						if (bPassesFilter)
						{
							if (!VirtualFolderFilter)
							{
								VirtualFolderFilter = &FilterList.FindOrAddFilter<FContentBrowserCompiledVirtualFolderFilter>();
							}

							if (!VirtualFolderFilter->CachedSubPaths.Contains(VirtualSubPath))
							{
								VirtualFolderFilter->CachedSubPaths.Add(VirtualSubPath, CreateVirtualFolderItem(VirtualSubPath));
							}
						}
					}
					return true;
				}, false);

				// Not recursive, virtual folder will not contain files
				ClassDataFilter.ValidFolders.Append(InternalPaths);
				return;
			}
		}
	}
	else if (bIncludeFiles)
	{
		if (InFilter.bRecursivePaths)
		{
			if (ConvertedPathType == EContentBrowserPathType::Internal)
			{
				// Nothing more to do, InternalPaths already contains ConvertedPath
			}
			else if (ConvertedPathType == EContentBrowserPathType::Virtual)
			{
				// Include all internal mounts that pass recursively
				RootPathVirtualTree.EnumerateSubPaths(InPath, [this, &InternalPaths, bIncludeEngine, bIncludePlugins](FName VirtualSubPath, FName InternalSubPath)
				{
					if (!InternalSubPath.IsNone())
					{
						if (RootClassPathPassesFilter(InternalSubPath, bIncludeEngine, bIncludePlugins))
						{
							InternalPaths.Add(InternalSubPath);
						}
					}
					return true;
				}, true);

				if (InternalPaths.Num() == 0)
				{
					// No internal folders found in the hierarchy of virtual path that passed, there will be no files
					return;
				}
			}
		}
		else
		{
			if (ConvertedPathType == EContentBrowserPathType::Internal)
			{
				// Nothing more to do, InternalPaths already contains ConvertedPath
			}
			else if (ConvertedPathType == EContentBrowserPathType::Virtual)
			{
				// There are no files directly contained by a dynamically generated fully virtual folder
				return;
			}
		}
	}

	if (InternalPaths.Num() == 0)
	{
		return;
	}

	FNativeClassHierarchyFilter ClassHierarchyFilter;
	ClassHierarchyFilter.ClassPaths.Reserve(InternalPaths.Num());
	for (const FName InternalPath : InternalPaths)
	{
		ClassHierarchyFilter.ClassPaths.Add(InternalPath);
	}
	ClassHierarchyFilter.bRecursivePaths = InFilter.bRecursivePaths;

	// Find the child class folders
	if (bIncludeFolders && !ClassHierarchyFilter.IsEmpty())
	{
		TArray<FString> ChildClassFolders;
		NativeClassHierarchy->GetMatchingFolders(ClassHierarchyFilter, ChildClassFolders);

		if (ConvertedPathType == EContentBrowserPathType::Virtual)
		{
			for (const FName InternalPath : InternalPaths)
			{
				ClassDataFilter.ValidFolders.Add(InternalPath);
			}
		}

		for (const FString& ChildClassFolder : ChildClassFolders)
		{
			ClassDataFilter.ValidFolders.Add(*ChildClassFolder);
		}
	}

	// If we are filtering all classes, then we can bail now as we won't return any file items
	if ((ClassFilter && (ClassFilter->ClassNamesToInclude.Num() > 0 && !ClassFilter->ClassNamesToInclude.Contains(TEXT("/Script/CoreUObject.Class")))) ||
		(ClassFilter && (ClassFilter->ClassNamesToExclude.Num() > 0 &&  ClassFilter->ClassNamesToExclude.Contains(TEXT("/Script/CoreUObject.Class")))) ||
		(ClassPermissionList && (ClassPermissionList->IsDenyListAll() || !ClassPermissionList->PassesFilter(TEXT("/Script/CoreUObject.Class"))))
		)
	{
		return;
	}

	// Find the child class files
	if (bIncludeFiles && !ClassHierarchyFilter.IsEmpty())
	{
		TArray<UClass*> ChildClassObjects;
		NativeClassHierarchy->GetMatchingClasses(ClassHierarchyFilter, ChildClassObjects);

		if (ChildClassObjects.Num() > 0)
		{
			TSet<FTopLevelAssetPath> ClassPathsToInclude;
			if (CollectionFilter)
			{
				TArray<FTopLevelAssetPath> ClassPathsForCollections;
				if (GetClassPathsForCollections(CollectionFilter->Collections, CollectionFilter->bIncludeChildCollections, ClassPathsForCollections) && ClassPathsForCollections.Num() == 0)
				{
					// If we had collections but they contained no classes then we can bail as nothing will pass the filter
					return;
				}

				ClassPathsToInclude.Append(ClassPathsForCollections);
			}

			for (UClass* ChildClassObject : ChildClassObjects)
			{
				const bool bPassesInclusiveFilter = ClassPathsToInclude.Num() == 0 || ClassPathsToInclude.Contains(FTopLevelAssetPath(ChildClassObject));
				const bool bPassesPermissionCheck = !ClassPermissionList || ClassPermissionList->PassesFilter(ChildClassObject->GetClassPathName().ToString());

				if (bPassesInclusiveFilter && bPassesPermissionCheck)
				{
					ClassDataFilter.ValidClasses.Add(ChildClassObject);
				}
			}
		}
	}
}

void UContentBrowserClassDataSource::EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
	const FContentBrowserDataFilterList* FilterList = InFilter.CompiledFilters.Find(this);
	if (!FilterList)
	{
		return;
	}
	
	const FContentBrowserCompiledClassDataFilter* ClassDataFilter = FilterList->FindFilter<FContentBrowserCompiledClassDataFilter>();
	if (!ClassDataFilter)
	{
		return;
	}

	if (EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders))
	{
		for (const FName& ValidFolder : ClassDataFilter->ValidFolders)
		{
			if (!InCallback(CreateClassFolderItem(ValidFolder)))
			{
				return;
			}
		}
	}

	if (EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles))
	{
		FNativeClassHierarchyGetClassPathCache Cache;
		for (UClass* ValidClass : ClassDataFilter->ValidClasses)
		{
			if (!InCallback(CreateClassFileItem(ValidClass, Cache)))
			{
				return;
			}
		}
	}

}

void UContentBrowserClassDataSource::EnumerateItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
	FName InternalPath;
	if (!TryConvertVirtualPathToInternal(InPath, InternalPath))
	{
		return;
	}
	
	ConditionalCreateNativeClassHierarchy();

	if (EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders))
	{
		if (TSharedPtr<const FNativeClassHierarchyNode> FolderNode = NativeClassHierarchy->FindNode(InternalPath, ENativeClassHierarchyNodeType::Folder))
		{
			InCallback(CreateClassFolderItem(InternalPath));
		}
	}

	if (EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles))
	{
		if (TSharedPtr<const FNativeClassHierarchyNode> ClassNode = NativeClassHierarchy->FindNode(InternalPath, ENativeClassHierarchyNodeType::Class))
		{
			InCallback(CreateClassFileItem(InternalPath, ClassNode));
		}
	}
}

void UContentBrowserClassDataSource::EnumerateItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
	if (!InPath.StartsWith(TEXT("/Script/")))
	{
		return;
	}

	const FTopLevelAssetPath Path(InPath);
	if (!Path.IsValid())
	{
		return;
	}

	if (EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders) && Path.GetAssetName().IsNone())
	{
		const FName FolderPath = NativeClassHierarchy->GetFolderPathForModule(FPackageName::GetShortFName(Path.GetPackageName()), NativeClassHierarchyGetClassPathCache.GameModules);
		if (TSharedPtr<const FNativeClassHierarchyNode> ClassNode = NativeClassHierarchy->FindNode(FolderPath, ENativeClassHierarchyNodeType::Folder))
		{
			InCallback(CreateClassFolderItem(FolderPath, ClassNode));
		}
	}

	if (EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles) && !Path.GetAssetName().IsNone())
	{
		if (UClass* Class = FindObject<UClass>(Path))
		{
			FString InternalPath;
			if (NativeClassHierarchy->GetClassPath(Class, InternalPath, NativeClassHierarchyGetClassPathCache.GameModules))
			{
				FName ClassPath = FName(InternalPath);
				if (TSharedPtr<const FNativeClassHierarchyNode> ClassNode = NativeClassHierarchy->FindNode(ClassPath, ENativeClassHierarchyNodeType::Class))
				{
					InCallback(CreateClassFileItem(ClassPath, ClassNode));
				}
			}
		}
	}
}

bool UContentBrowserClassDataSource::EnumerateItemsForObjects(const TArrayView<UObject*> InObjects, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
	ConditionalCreateNativeClassHierarchy();

	FString InternalPath;
	for (UObject* InObject : InObjects)
	{
		if (UClass* InClass = Cast<UClass>(InObject))
		{
			InternalPath.Reset();
			if (NativeClassHierarchy->GetClassPath(InClass, InternalPath, NativeClassHierarchyGetClassPathCache.GameModules))
			{
				FName ClassPath = FName(InternalPath);
				if (TSharedPtr<const FNativeClassHierarchyNode> ClassNode = NativeClassHierarchy->FindNode(ClassPath, ENativeClassHierarchyNodeType::Class))
				{
					if (!InCallback(CreateClassFileItem(ClassPath, ClassNode)))
					{
						return false;
					}
				}
			}
		}
	}

	return true;
}

bool UContentBrowserClassDataSource::IsFolderVisible(const FName InPath, const EContentBrowserIsFolderVisibleFlags InFlags, const FContentBrowserFolderContentsFilter& InContentsFilter)
{
	// We only contain classes, bail if the caller wants to filter out folders that contain only classes
	if (!EnumHasAnyFlags(InContentsFilter.ItemCategoryFilter, EContentBrowserItemCategoryFilter::IncludeClasses))
	{
		return false;
	}

	FName ConvertedPath;
	const EContentBrowserPathType ConvertedPathType = TryConvertVirtualPath(InPath, ConvertedPath);
	if (ConvertedPathType == EContentBrowserPathType::Internal)
	{
		if (!IsKnownClassPath(ConvertedPath))
		{
			return false;
		}
	}
	else if (ConvertedPathType == EContentBrowserPathType::Virtual)
	{
		return true;
	}
	else
	{
		return false;
	}

	ConditionalCreateNativeClassHierarchy();

	if (ContentBrowserDataUtils::IsTopLevelFolder(ConvertedPath))
	{
		return true;
	}

	// Class flag was checked above - if we are filtering out folders that don't contain "class" elements, we are filtering out all empty folders from this provider
	if (!NativeClassHierarchy->HasClasses(ConvertedPath, /*bRecursive*/ true))
	{
		return false;
	}
	return true;
}

bool UContentBrowserClassDataSource::DoesItemPassFilter(const FContentBrowserItemData& InItem, const FContentBrowserDataCompiledFilter& InFilter)
{
	const FContentBrowserDataFilterList* FilterList = InFilter.CompiledFilters.Find(this);
	if (!FilterList)
	{
		return false;
	}

	const FContentBrowserCompiledClassDataFilter* ClassDataFilter = FilterList->FindFilter<FContentBrowserCompiledClassDataFilter>();
	if (!ClassDataFilter)
	{
		return false;
	}

	switch (InItem.GetItemType())
	{
	case EContentBrowserItemFlags::Type_Folder:
		if (EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders) && ClassDataFilter->ValidFolders.Num() > 0)
		{
			if (TSharedPtr<const FContentBrowserClassFolderItemDataPayload> FolderPayload = GetClassFolderItemPayload(InItem))
			{
				return ClassDataFilter->ValidFolders.Contains(FolderPayload->GetInternalPath());
			}
		}
		break;

	case EContentBrowserItemFlags::Type_File:
		if (EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles) && ClassDataFilter->ValidClasses.Num() > 0)
		{
			if (TSharedPtr<const FContentBrowserClassFileItemDataPayload> ClassPayload = GetClassFileItemPayload(InItem))
			{
				return ClassDataFilter->ValidClasses.Contains(ClassPayload->GetClass());
			}
		}
		break;

	default:
		break;
	}
	
	return false;
}

bool UContentBrowserClassDataSource::GetItemAttribute(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	return ContentBrowserClassData::GetItemAttribute(GetClassTypeActions().Get(), this, InItem, InIncludeMetaData, InAttributeKey, OutAttributeValue);
}

bool UContentBrowserClassDataSource::GetItemAttributes(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues)
{
	return ContentBrowserClassData::GetItemAttributes(this, InItem, InIncludeMetaData, OutAttributeValues);
}

bool UContentBrowserClassDataSource::GetItemPhysicalPath(const FContentBrowserItemData& InItem, FString& OutDiskPath)
{
	return ContentBrowserClassData::GetItemPhysicalPath(this, InItem, OutDiskPath);
}

bool UContentBrowserClassDataSource::CanEditItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return ContentBrowserClassData::CanEditItem(this, InItem, OutErrorMsg);
}

bool UContentBrowserClassDataSource::EditItem(const FContentBrowserItemData& InItem)
{
	return ContentBrowserClassData::EditItems(GetClassTypeActions().Get(), this, MakeArrayView(&InItem, 1));
}

bool UContentBrowserClassDataSource::BulkEditItems(TArrayView<const FContentBrowserItemData> InItems)
{
	return ContentBrowserClassData::EditItems(GetClassTypeActions().Get(), this, InItems);
}

bool UContentBrowserClassDataSource::AppendItemReference(const FContentBrowserItemData& InItem, FString& InOutStr)
{
	return ContentBrowserClassData::AppendItemReference(this, InItem, InOutStr);
}

bool UContentBrowserClassDataSource::AppendItemObjectPath(const FContentBrowserItemData& InItem, FString& InOutStr)
{
	return ContentBrowserClassData::AppendItemObjectPath(this, InItem, InOutStr);
}

bool UContentBrowserClassDataSource::AppendItemPackageName(const FContentBrowserItemData& InItem, FString& InOutStr)
{
	return ContentBrowserClassData::AppendItemPackageName(this, InItem, InOutStr);
}

bool UContentBrowserClassDataSource::UpdateThumbnail(const FContentBrowserItemData& InItem, FAssetThumbnail& InThumbnail)
{
	return ContentBrowserClassData::UpdateItemThumbnail(this, InItem, InThumbnail);
}

bool UContentBrowserClassDataSource::TryGetCollectionId(const FContentBrowserItemData& InItem, FSoftObjectPath& OutCollectionId)
{
	if (TSharedPtr<const FContentBrowserClassFileItemDataPayload> ClassPayload = GetClassFileItemPayload(InItem))
	{
		OutCollectionId = FSoftObjectPath(ClassPayload->GetAssetData().GetSoftObjectPath());
		return true;
	}
	return false;
}

bool UContentBrowserClassDataSource::Legacy_TryGetPackagePath(const FContentBrowserItemData& InItem, FName& OutPackagePath)
{
	return false;
}

bool UContentBrowserClassDataSource::Legacy_TryGetAssetData(const FContentBrowserItemData& InItem, FAssetData& OutAssetData)
{
	if (TSharedPtr<const FContentBrowserClassFileItemDataPayload> ClassPayload = GetClassFileItemPayload(InItem))
	{
		OutAssetData = ClassPayload->GetAssetData();
		return true;
	}
	return false;
}

bool UContentBrowserClassDataSource::Legacy_TryConvertPackagePathToVirtualPath(const FName InPackagePath, FName& OutPath)
{
	return false;
}

bool UContentBrowserClassDataSource::Legacy_TryConvertAssetDataToVirtualPath(const FAssetData& InAssetData, const bool InUseFolderPaths, FName& OutPath)
{
	return (InAssetData.AssetClassPath == FTopLevelAssetPath(TEXT("/Script/CoreUObject"), TEXT("Class"))) // Ignore non-class class items
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		&& TryConvertInternalPathToVirtual(InUseFolderPaths ? InAssetData.PackagePath : InAssetData.ObjectPath, OutPath);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UContentBrowserClassDataSource::IsKnownClassPath(const FName InPackagePath) const
{
	FNameBuilder PackagePathStr(InPackagePath);
	return FStringView(PackagePathStr).StartsWith(TEXT("/Classes_"));
}

bool UContentBrowserClassDataSource::GetClassPathsForCollections(TArrayView<const FCollectionRef> InCollections, const bool bIncludeChildCollections, TArray<FTopLevelAssetPath>& OutClassPaths)
{
	if (InCollections.Num() > 0)
	{
		const ECollectionRecursionFlags::Flags CollectionRecursionMode = bIncludeChildCollections ? ECollectionRecursionFlags::SelfAndChildren : ECollectionRecursionFlags::Self;
		
		for (const FCollectionRef& Collection : InCollections)
		{
			if (Collection.Container)
			{
				Collection.Container->GetClassesInCollection(Collection.Name, Collection.Type, OutClassPaths, CollectionRecursionMode);
			}
		}

		return true;
	}

	return false;
}

FContentBrowserItemData UContentBrowserClassDataSource::CreateClassFolderItem(const FName InFolderPath)
{
	TSharedPtr<const FNativeClassHierarchyNode> FolderNode = NativeClassHierarchy->FindNode(InFolderPath, ENativeClassHierarchyNodeType::Folder);

	return CreateClassFolderItem(InFolderPath, FolderNode);
}

FContentBrowserItemData UContentBrowserClassDataSource::CreateClassFolderItem(const FName InFolderPath, const TSharedPtr<const FNativeClassHierarchyNode>& InFolderNode)
{
	FName VirtualizedPath;
	TryConvertInternalPathToVirtual(InFolderPath, VirtualizedPath);

	return ContentBrowserClassData::CreateClassFolderItem(this, VirtualizedPath, InFolderPath, InFolderNode->LoadedFrom.IsSet());
}

FContentBrowserItemData UContentBrowserClassDataSource::CreateClassFileItem(UClass* InClass, FNativeClassHierarchyGetClassPathCache& InCache)
{
	ConditionalCreateNativeClassHierarchy();

	FName ClassPath;
	{
		FString ClassPathStr;
		const bool bValidClassPath = NativeClassHierarchy->GetClassPath(InClass, ClassPathStr, InCache.GameModules);
		checkf(bValidClassPath, TEXT("GetClassPath failed to return a result for '%s'"), *InClass->GetPathName());
		ClassPath = *ClassPathStr;
	}

	TSharedPtr<const FNativeClassHierarchyNode> ClassNode = NativeClassHierarchy->FindNode(ClassPath, ENativeClassHierarchyNodeType::Class);

	return CreateClassFileItem(ClassPath, ClassNode);
}

FContentBrowserItemData UContentBrowserClassDataSource::CreateClassFileItem(const FName InClassPath, const TSharedPtr<const FNativeClassHierarchyNode>& InClassNode)
{
	FName VirtualizedPath;
	TryConvertInternalPathToVirtual(InClassPath, VirtualizedPath);

	return ContentBrowserClassData::CreateClassFileItem(this, VirtualizedPath, InClassPath, InClassNode->Class, InClassNode->LoadedFrom.IsSet());
}

TSharedPtr<const FContentBrowserClassFolderItemDataPayload> UContentBrowserClassDataSource::GetClassFolderItemPayload(const FContentBrowserItemData& InItem) const
{
	return ContentBrowserClassData::GetClassFolderItemPayload(this, InItem);
}

TSharedPtr<const FContentBrowserClassFileItemDataPayload> UContentBrowserClassDataSource::GetClassFileItemPayload(const FContentBrowserItemData& InItem) const
{
	return ContentBrowserClassData::GetClassFileItemPayload(this, InItem);
}

void UContentBrowserClassDataSource::PopulateAddNewContextMenu(UToolMenu* InMenu)
{
	if (ensure(GUnrealEd) && !GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed())
	{
		return;
	}

	const UContentBrowserDataMenuContext_AddNewMenu* ContextObject = InMenu->FindContext<UContentBrowserDataMenuContext_AddNewMenu>();
	checkf(ContextObject, TEXT("Required context UContentBrowserDataMenuContext_AddNewMenu was missing!"));

	// Extract the internal class paths that belong to this data source from the full list of selected paths given in the context
	TArray<FName> SelectedClassPaths;
	for (const FName& SelectedPath : ContextObject->SelectedPaths)
	{
		FName InternalPath;
		if (TryConvertVirtualPathToInternal(SelectedPath, InternalPath) && IsKnownClassPath(InternalPath))
		{
			SelectedClassPaths.Add(InternalPath);
		}
	}

	// Only add the asset items if we have a class path selected
	FNewClassContextMenu::FOnNewClassRequested OnNewClassRequested;
	if (SelectedClassPaths.Num() > 0)
	{
		OnNewClassRequested = FNewClassContextMenu::FOnNewClassRequested::CreateUObject(this, &UContentBrowserClassDataSource::OnNewClassRequested);
	}

	FNewClassContextMenu::MakeContextMenu(
		InMenu,
		SelectedClassPaths,
		OnNewClassRequested
		);
}

void UContentBrowserClassDataSource::OnNewClassRequested(const FName InSelectedPath)
{
	ConditionalCreateNativeClassHierarchy();

	// Parse out the on disk location for the currently selected path, this will then be used as the default location for the new class (if a valid project module location)
	FString ExistingFolderPath;
	if (!InSelectedPath.IsNone())
	{
		NativeClassHierarchy->GetFileSystemPath(InSelectedPath.ToString(), ExistingFolderPath);
	}

	FGameProjectGenerationModule::Get().OpenAddCodeToProjectDialog(
		FAddToProjectConfig()
		.InitialPath(ExistingFolderPath)
		.ParentWindow(FGlobalTabmanager::Get()->GetRootWindow())
		);
}

void UContentBrowserClassDataSource::ConditionalCreateNativeClassHierarchy()
{
	if (!NativeClassHierarchy)
	{
		NativeClassHierarchy = MakeShared<FNativeClassHierarchy>();

		NativeClassHierarchy->OnClassesAdded().AddUObject(this, &UContentBrowserClassDataSource::OnClassesAdded);
		NativeClassHierarchy->OnClassesRemoved().AddUObject(this, &UContentBrowserClassDataSource::OnClassesRemoved);
		NativeClassHierarchy->OnFoldersAdded().AddUObject(this, &UContentBrowserClassDataSource::OnFoldersAdded);
		NativeClassHierarchy->OnFoldersRemoved().AddUObject(this, &UContentBrowserClassDataSource::OnFoldersRemoved);
	}
}

void UContentBrowserClassDataSource::OnFoldersAdded(const TArrayView<TSharedRef<const FNativeClassHierarchyNode>> InFolders)
{
	NativeClassHierarchyGetClassPathCache.Reset();
	SetVirtualPathTreeNeedsRebuild();

	for (const TSharedRef<const FNativeClassHierarchyNode>& Folder : InFolders)
	{
		QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemAddedUpdate(CreateClassFolderItem(*Folder->EntryPath, Folder)));
	}
}

void UContentBrowserClassDataSource::OnFoldersRemoved(const TArrayView<TSharedRef<const FNativeClassHierarchyNode>> InFolders)
{
	NativeClassHierarchyGetClassPathCache.Reset();
	SetVirtualPathTreeNeedsRebuild();

	for (const TSharedRef<const FNativeClassHierarchyNode>& Folder : InFolders)
	{
		QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemRemovedUpdate(CreateClassFolderItem(*Folder->EntryPath, Folder)));
	}
}

void UContentBrowserClassDataSource::OnClassesAdded(const TArrayView<TSharedRef<const FNativeClassHierarchyNode>> InClasses)
{
	for (const TSharedRef<const FNativeClassHierarchyNode>& Class : InClasses)
	{
		QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemAddedUpdate(CreateClassFileItem(*Class->EntryPath, Class)));
	}
}

void UContentBrowserClassDataSource::OnClassesRemoved(const TArrayView<TSharedRef<const FNativeClassHierarchyNode>> InClasses)
{
	for (const TSharedRef<const FNativeClassHierarchyNode>& Class : InClasses)
	{
		QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemRemovedUpdate(CreateClassFileItem(*Class->EntryPath, Class)));
	}
}

#undef LOCTEXT_NAMESPACE

