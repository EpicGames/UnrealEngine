// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "UObject/StrongObjectPtr.h"
#include "ContentBrowserClassDataSource.h"

class FContentBrowserClassDataSourceModule : public FDefaultModuleImpl
{
public:
	virtual void StartupModule() override
	{
		ClassDataSource.Reset(NewObject<UContentBrowserClassDataSource>(GetTransientPackage(), "ClassData"));
		ClassDataSource->Initialize();
	}

	virtual void ShutdownModule() override
	{
		ClassDataSource.Reset();
	}

private:
	TStrongObjectPtr<UContentBrowserClassDataSource> ClassDataSource;
};

IMPLEMENT_MODULE(FContentBrowserClassDataSourceModule, ContentBrowserClassDataSource);
