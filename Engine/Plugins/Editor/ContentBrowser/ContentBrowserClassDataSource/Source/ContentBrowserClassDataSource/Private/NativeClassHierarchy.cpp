// Copyright Epic Games, Inc. All Rights Reserved.

#include "NativeClassHierarchy.h"

#include "Interfaces/IPluginManager.h"
#include "Interfaces/IProjectManager.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/App.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProjectDescriptor.h"
#include "SourceCodeNavigation.h"
#include "String/ParseTokens.h"
#include "UObject/Package.h"
#include "UObject/Reload.h"
#include "UObject/UObjectIterator.h"

DEFINE_LOG_CATEGORY_STATIC(LogNativeClassHierarchy, Log, All);

TSharedRef<FNativeClassHierarchyNode> FNativeClassHierarchyNode::MakeFolderEntry(FName InEntryName, FString InEntryPath, TOptional<EPluginLoadedFrom> LoadedFrom)
{
	TSharedRef<FNativeClassHierarchyNode> NewEntry = MakeShareable(new FNativeClassHierarchyNode());
	NewEntry->Type = ENativeClassHierarchyNodeType::Folder;
	NewEntry->Class = nullptr;
	NewEntry->EntryName = InEntryName;
	NewEntry->EntryPath = MoveTemp(InEntryPath);
	NewEntry->LoadedFrom = LoadedFrom;
	return NewEntry;
}

TSharedRef<FNativeClassHierarchyNode> FNativeClassHierarchyNode::MakeClassEntry(UClass* InClass, FName InClassModuleName, FString InClassModuleRelativePath, FString InEntryPath, TOptional<EPluginLoadedFrom> LoadedFrom)
{
	TSharedRef<FNativeClassHierarchyNode> NewEntry = MakeShareable(new FNativeClassHierarchyNode());
	NewEntry->Type = ENativeClassHierarchyNodeType::Class;
	NewEntry->Class = InClass;
	NewEntry->ClassModuleName = MoveTemp(InClassModuleName);
	NewEntry->ClassModuleRelativePath = MoveTemp(InClassModuleRelativePath);
	NewEntry->EntryName = InClass->GetFName();
	NewEntry->EntryPath = MoveTemp(InEntryPath);
	NewEntry->LoadedFrom = LoadedFrom;
	return NewEntry;
}

void FNativeClassHierarchyNode::AddChild(TSharedRef<FNativeClassHierarchyNode> ChildEntry)
{
	check(Type == ENativeClassHierarchyNodeType::Folder);
	Children.Add(FNativeClassHierarchyNodeKey(ChildEntry->EntryName, ChildEntry->Type), MoveTemp(ChildEntry));
}

FNativeClassHierarchy::FNativeClassHierarchy()
{
	PopulateHierarchy();

	// Register to be notified of module changes
	FModuleManager::Get().OnModulesChanged().AddRaw(this, &FNativeClassHierarchy::OnModulesChanged);

	// Register to be notified of reloads
	FCoreUObjectDelegates::ReloadReinstancingCompleteDelegate.AddRaw(this, &FNativeClassHierarchy::OnReloadReinstancingComplete);
	FCoreUObjectDelegates::ReloadAddedClassesDelegate.AddRaw(this, &FNativeClassHierarchy::OnReloadClassesAdded);
}

FNativeClassHierarchy::~FNativeClassHierarchy()
{
	FModuleManager::Get().OnModulesChanged().RemoveAll(this);

	FCoreUObjectDelegates::ReloadReinstancingCompleteDelegate.RemoveAll(this);
	FCoreUObjectDelegates::ReloadAddedClassesDelegate.RemoveAll(this);
}

TSharedPtr<const FNativeClassHierarchyNode> FNativeClassHierarchy::FindNode(const FName InClassPath, const ENativeClassHierarchyNodeType InType) const
{
	TArray<TSharedRef<FNativeClassHierarchyNode>, TInlineAllocator<4>> NodesFound;
	GatherMatchingNodesForPaths(TArrayView<const FName>(&InClassPath, 1), NodesFound, InType);

	for(const auto& NodeFound : NodesFound)
	{
		if(NodeFound->Type == InType)
		{
			return NodeFound;
		}
	}

	return nullptr;
}

bool FNativeClassHierarchy::RootNodePassesFilter(const FName InRootName, const TSharedPtr<const FNativeClassHierarchyNode>& InRootNode, const bool bIncludeEngineClasses, const bool bIncludePluginClasses) const
{
	static const FName EngineRootNodeName = "Classes_Engine";
	static const FName GameRootNodeName = "Classes_Game";

	return (InRootName == GameRootNodeName) ||								// Always include game classes
			(InRootName == EngineRootNodeName && bIncludeEngineClasses) ||  	// Only include engine classes if we were asked for them
			(bIncludePluginClasses && InRootNode->LoadedFrom == EPluginLoadedFrom::Project) || // Only include Game plugin classes if we were asked for them
			(bIncludePluginClasses && bIncludeEngineClasses && InRootNode->LoadedFrom == EPluginLoadedFrom::Engine); //  Only include engine plugin classes if we were asked for them
}

bool FNativeClassHierarchy::RootNodePassesFilter(const FName InRootName, const bool bIncludeEngineClasses, const bool bIncludePluginClasses) const
{
	if (const TSharedPtr<FNativeClassHierarchyNode>* Found = RootNodes.Find(InRootName))
	{
		return RootNodePassesFilter(InRootName, *Found, bIncludeEngineClasses, bIncludePluginClasses);
	}

	return false;
}

bool FNativeClassHierarchy::HasClasses(const FName InClassPath, const bool bRecursive) const
{
	TArray<TSharedRef<FNativeClassHierarchyNode>, TInlineAllocator<4>> NodesToSearch;
	GatherMatchingNodesForPaths(TArrayView<const FName>(&InClassPath, 1), NodesToSearch);

	for(const auto& NodeToSearch : NodesToSearch)
	{
		if(HasClassesRecursive(NodeToSearch, bRecursive))
		{
			return true;
		}
	}

	return false;
}

bool FNativeClassHierarchy::HasFolders(const FName InClassPath, const bool bRecursive) const
{
	TArray<TSharedRef<FNativeClassHierarchyNode>, TInlineAllocator<4>> NodesToSearch;
	GatherMatchingNodesForPaths(TArrayView<const FName>(&InClassPath, 1), NodesToSearch);

	for(const auto& NodeToSearch : NodesToSearch)
	{
		if(HasFoldersRecursive(NodeToSearch, bRecursive))
		{
			return true;
		}
	}

	return false;
}

void FNativeClassHierarchy::GetMatchingClasses(const FNativeClassHierarchyFilter& Filter, TArray<UClass*>& OutClasses) const
{
	TArray<TSharedRef<FNativeClassHierarchyNode>, TInlineAllocator<4>> NodesToSearch;
	GatherMatchingNodesForPaths(Filter.ClassPaths, NodesToSearch);

	for(const auto& NodeToSearch : NodesToSearch)
	{
		GetClassesRecursive(NodeToSearch, OutClasses, Filter.bRecursivePaths);
	}
}

void FNativeClassHierarchy::GetMatchingFolders(const FNativeClassHierarchyFilter& Filter, TArray<FString>& OutFolders) const
{
	TArray<TSharedRef<FNativeClassHierarchyNode>, TInlineAllocator<4>> NodesToSearch;
	GatherMatchingNodesForPaths(Filter.ClassPaths, NodesToSearch);

	for(const auto& NodeToSearch : NodesToSearch)
	{
		GetFoldersRecursive(NodeToSearch, OutFolders, Filter.bRecursivePaths);
	}
}

void FNativeClassHierarchy::GetClassRoots(TArray<FName>& OutClassRoots, const bool bIncludeEngineClasses, const bool bIncludePluginClasses) const
{
	static const FName EngineRootNodeName = "Classes_Engine";
	static const FName GameRootNodeName = "Classes_Game";

	for(const auto& RootNode : RootNodes)
	{
		bool bRootNodePassesFilter =
			(RootNode.Key == GameRootNodeName) ||								// Always include game classes
			(RootNode.Key == EngineRootNodeName && bIncludeEngineClasses) ||  	// Only include engine classes if we were asked for them
			(bIncludePluginClasses && RootNode.Value->LoadedFrom == EPluginLoadedFrom::Project) || // Only include Game plugin classes if we were asked for them
			(bIncludePluginClasses && bIncludeEngineClasses && RootNode.Value->LoadedFrom == EPluginLoadedFrom::Engine); //  Only include engine plugin classes if we were asked for them

		if(bRootNodePassesFilter)
		{
			OutClassRoots.Add(*RootNode.Value->EntryPath);
		}
	}
}

void FNativeClassHierarchy::GetClassFolders(TArray<FName>& OutClassRoots, TArray<FString>& OutClassFolders, const bool bIncludeEngineClasses, const bool bIncludePluginClasses) const
{
	static const FName EngineRootNodeName = "Classes_Engine";
	static const FName GameRootNodeName = "Classes_Game";

	for(const auto& RootNode : RootNodes)
	{
		bool bRootNodePassesFilter =
			(RootNode.Key == GameRootNodeName) ||								// Always include game classes
			(RootNode.Key == EngineRootNodeName && bIncludeEngineClasses) ||  	// Only include engine classes if we were asked for them
			(bIncludePluginClasses && RootNode.Value->LoadedFrom == EPluginLoadedFrom::Project) || // Only include Game plugin classes if we were asked for them
			(bIncludePluginClasses && bIncludeEngineClasses && RootNode.Value->LoadedFrom == EPluginLoadedFrom::Engine); //  Only include engine plugin classes if we were asked for them

		if(bRootNodePassesFilter)
		{
			OutClassRoots.Add(*RootNode.Value->EntryPath);
			GetFoldersRecursive(RootNode.Value.ToSharedRef(), OutClassFolders);
		}
	}
}

bool FNativeClassHierarchy::HasClassesRecursive(const TSharedRef<FNativeClassHierarchyNode>& HierarchyNode, const bool bRecurse)
{
	for(const auto& ChildNode : HierarchyNode->Children)
	{
		if(ChildNode.Value->Type == ENativeClassHierarchyNodeType::Class)
		{
			return true;
		}

		if(bRecurse && HasClassesRecursive(ChildNode.Value.ToSharedRef()))
		{
			return true;
		}
	}

	return false;
}

bool FNativeClassHierarchy::HasFoldersRecursive(const TSharedRef<FNativeClassHierarchyNode>& HierarchyNode, const bool bRecurse)
{
	for(const auto& ChildNode : HierarchyNode->Children)
	{
		if(ChildNode.Value->Type == ENativeClassHierarchyNodeType::Folder)
		{
			return true;
		}

		if(bRecurse && HasFoldersRecursive(ChildNode.Value.ToSharedRef()))
		{
			return true;
		}
	}

	return false;
}

void FNativeClassHierarchy::GetClassesRecursive(const TSharedRef<FNativeClassHierarchyNode>& HierarchyNode, TArray<UClass*>& OutClasses, const bool bRecurse)
{
	for(const auto& ChildNode : HierarchyNode->Children)
	{
		if(ChildNode.Value->Type == ENativeClassHierarchyNodeType::Class)
		{
			OutClasses.Add(ChildNode.Value->Class);
		}

		if(bRecurse)
		{
			GetClassesRecursive(ChildNode.Value.ToSharedRef(), OutClasses);
		}
	}
}

void FNativeClassHierarchy::GetFoldersRecursive(const TSharedRef<FNativeClassHierarchyNode>& HierarchyNode, TArray<FString>& OutFolders, const bool bRecurse)
{
	for(const auto& ChildNode : HierarchyNode->Children)
	{
		if(ChildNode.Value->Type == ENativeClassHierarchyNodeType::Folder)
		{
			OutFolders.Add(ChildNode.Value->EntryPath);
		}

		if(bRecurse)
		{
			GetFoldersRecursive(ChildNode.Value.ToSharedRef(), OutFolders);
		}
	}
}

void FNativeClassHierarchy::GatherMatchingNodesForPaths(const TArrayView<const FName>& InClassPaths, TArray<TSharedRef<FNativeClassHierarchyNode>, TInlineAllocator<4>>& OutMatchingNodes, const ENativeClassHierarchyNodeType InType) const
{
	if(InClassPaths.Num() == 0)
	{
		// No paths means search all roots
		OutMatchingNodes.Reserve(RootNodes.Num());
		for(const auto& RootNode : RootNodes)
		{
			OutMatchingNodes.Add(RootNode.Value.ToSharedRef());
		}
	}
	else
	{
		for(const FName& ClassPath : InClassPaths)
		{
			TSharedPtr<FNativeClassHierarchyNode> CurrentNode;

			TArray<FString> ClassPathParts;
			ClassPath.ToString().ParseIntoArray(ClassPathParts, TEXT("/"), true);
			for(const FString& ClassPathPart : ClassPathParts)
			{
				// Try and find the node associated with this part of the path...
				const FName ClassPathPartName = *ClassPathPart;

				// Compare the address of the strings to see if it is the last part
				if (InType == ENativeClassHierarchyNodeType::Class && &ClassPathPart == &ClassPathParts.Last())
				{

					CurrentNode = (CurrentNode.IsValid()) ? CurrentNode->Children.FindRef(FNativeClassHierarchyNodeKey(ClassPathPartName, ENativeClassHierarchyNodeType::Class)) : RootNodes.FindRef(ClassPathPartName);
				}
				else
				{
					CurrentNode = (CurrentNode.IsValid()) ? CurrentNode->Children.FindRef(FNativeClassHierarchyNodeKey(ClassPathPartName, ENativeClassHierarchyNodeType::Folder)) : RootNodes.FindRef(ClassPathPartName);
				}

				// ... bail out if we didn't find a valid node
				if(!CurrentNode.IsValid())
				{
					break;
				}
			}

			if(CurrentNode.IsValid())
			{
				OutMatchingNodes.Add(CurrentNode.ToSharedRef());
			}
		}
	}
}

void FNativeClassHierarchy::PopulateHierarchy()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNativeClassHierarchy::PopulateHierarchy);

	double StartTime = FPlatformTime::Seconds();

	FClassChanges ClassChanges;

	RootNodes.Empty();

	TSet<FName> GameModules = GetGameModules();
	
	for(TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass* const CurrentClass = *ClassIt;
		AddClass(CurrentClass, GameModules, ClassChanges);
	}

	UE_LOGF(LogNativeClassHierarchy, Verbose, "Native class hierarchy populated in %0.4f seconds. Added %d classes and %d folders.", FPlatformTime::Seconds() - StartTime, ClassChanges.ClassesModified.Num(), ClassChanges.FoldersModified.Num());

	if (!ClassChanges.FoldersModified.IsEmpty())
	{
		FoldersAddedDelegate.Broadcast(ClassChanges.FoldersModified);
	}
	if (!ClassChanges.ClassesModified.IsEmpty())
	{
		ClassesAddedDelegate.Broadcast(ClassChanges.ClassesModified);
	}
}

void FNativeClassHierarchy::AddClassesForModule(const FName& InModuleName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNativeClassHierarchy::AddClassesForModule);

	double StartTime = FPlatformTime::Seconds();
	FClassChanges ClassChanges;

	// Find the class package for this module
	UPackage* const ClassPackage = FindPackage(nullptr, *(FString("/Script/") + InModuleName.ToString()));
	if(!ClassPackage)
	{
		return;
	}

	TSet<FName> GameModules = GetGameModules();
	
	TArray<UObject*> PackageObjects;
	GetObjectsWithOuter(ClassPackage, PackageObjects, EGetObjectsFlags::None);
	ClassChanges.ClassesModified.Reserve(PackageObjects.Num());

	for(UObject* Object : PackageObjects)
	{
		UClass* const CurrentClass = Cast<UClass>(Object);
		if(CurrentClass)
		{
			AddClass(CurrentClass, GameModules, ClassChanges);
		}
	}

	UE_LOGF(LogNativeClassHierarchy, Verbose, "Native class hierarchy updated for '%ls' in %0.4f seconds. Added %d classes and %d folders.", *InModuleName.ToString(), FPlatformTime::Seconds() - StartTime, ClassChanges.ClassesModified.Num(), ClassChanges.FoldersModified.Num());

	if (!ClassChanges.FoldersModified.IsEmpty())
	{
		FoldersAddedDelegate.Broadcast(ClassChanges.FoldersModified);
	}
	if (!ClassChanges.ClassesModified.IsEmpty())
	{
		ClassesAddedDelegate.Broadcast(ClassChanges.ClassesModified);
	}
}

void FNativeClassHierarchy::RemoveClassesForModule(const FName& InModuleName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNativeClassHierarchy::RemoveClassesForModule);

	// Modules always exist directly under a root
	for(const auto& RootNode : RootNodes)
	{
		TSharedPtr<FNativeClassHierarchyNode> ModuleNode = RootNode.Value->Children.FindRef(FNativeClassHierarchyNodeKey(InModuleName, ENativeClassHierarchyNodeType::Folder));
		if(ModuleNode.IsValid())
		{
			FClassChanges ClassChanges;

			// If this module was the only child of this root, then we need to remove the root as well
			if(RootNode.Value->Children.Num() == 1)
			{
				PopulateClassChanges(RootNode.Value, ClassChanges);
				RootNodes.Remove(RootNode.Key);
			}
			else
			{
				// Remove this module from its root
				TSharedPtr<FNativeClassHierarchyNode> RemovedNode;
				RootNode.Value->Children.RemoveAndCopyValue(FNativeClassHierarchyNodeKey(InModuleName, ENativeClassHierarchyNodeType::Folder), RemovedNode);
				PopulateClassChanges(RemovedNode, ClassChanges);
			}

			if (!ClassChanges.ClassesModified.IsEmpty())
			{
				ClassesRemovedDelegate.Broadcast(ClassChanges.ClassesModified);
			}
			if (!ClassChanges.FoldersModified.IsEmpty())
			{
				FoldersRemovedDelegate.Broadcast(ClassChanges.FoldersModified);
			}

			// We've found the module - break
			break;
		}
	}
}

void FNativeClassHierarchy::AddClass(UClass* InClass, const TSet<FName>& InGameModules, FClassChanges& ClassChanges)
{
	// Ignore deprecated and temporary classes
	if(InClass->HasAnyClassFlags(CLASS_Deprecated | CLASS_NewerVersionExists) || FKismetEditorUtilities::IsClassABlueprintSkeleton(InClass))
	{
		return;
	}

	const FName ClassModuleName = GetClassModuleName(InClass);
	if(ClassModuleName.IsNone())
	{
		return;
	}

	static const FName ModuleRelativePathMetaDataKey = "ModuleRelativePath";
	const FString& ClassModuleRelativeIncludePath = InClass->GetMetaData(ModuleRelativePathMetaDataKey);
	if(ClassModuleRelativeIncludePath.IsEmpty())
	{
		return;
	}

	// Work out which root this class should go under
	TOptional<EPluginLoadedFrom> WhereLoadedFrom;
	const FName RootNodeName = GetClassPathRootForModule(ClassModuleName, InGameModules, WhereLoadedFrom);

	// Work out the final path to this class within the hierarchy (which isn't the same as the path on disk)
	const FString ClassModuleRelativePath = ClassModuleRelativeIncludePath.Left(ClassModuleRelativeIncludePath.Find(TEXT("/"), ESearchCase::CaseSensitive, ESearchDir::FromEnd));
	const FString ClassHierarchyPath = ClassModuleName.ToString() + TEXT("/") + ClassModuleRelativePath;

	// Ensure we've added a valid root node
	TSharedPtr<FNativeClassHierarchyNode>& RootNode = RootNodes.FindOrAdd(RootNodeName);
	if(!RootNode.IsValid())
	{
		RootNode = FNativeClassHierarchyNode::MakeFolderEntry(RootNodeName, TEXT("/") + RootNodeName.ToString(), WhereLoadedFrom);
		RootNode->LoadedFrom = WhereLoadedFrom;
		ClassChanges.FoldersModified.Add(RootNode.ToSharedRef());
	}

	// Split the class path and ensure we have nodes for each part
	TArray<FString> HierarchyPathParts;
	ClassHierarchyPath.ParseIntoArray(HierarchyPathParts, TEXT("/"), true);
	TSharedPtr<FNativeClassHierarchyNode> CurrentNode = RootNode;
	for(const FString& HierarchyPathPart : HierarchyPathParts)
	{
		const FName HierarchyPathPartName = *HierarchyPathPart;
		TSharedPtr<FNativeClassHierarchyNode>& ChildNode = CurrentNode->Children.FindOrAdd(FNativeClassHierarchyNodeKey(HierarchyPathPartName, ENativeClassHierarchyNodeType::Folder));
		if(!ChildNode.IsValid())
		{
			ChildNode = FNativeClassHierarchyNode::MakeFolderEntry(HierarchyPathPartName, CurrentNode->EntryPath + TEXT("/") + HierarchyPathPart, WhereLoadedFrom);
			ClassChanges.FoldersModified.Add(ChildNode.ToSharedRef());
		}
		CurrentNode = ChildNode;
	}

	// Now add the final entry for the class
	TSharedRef<FNativeClassHierarchyNode> ClassNode = FNativeClassHierarchyNode::MakeClassEntry(InClass, ClassModuleName, ClassModuleRelativePath, CurrentNode->EntryPath + TEXT("/") + InClass->GetName(), WhereLoadedFrom);
	ClassChanges.ClassesModified.Add(ClassNode);
	CurrentNode->AddChild(MoveTemp(ClassNode));
}

void FNativeClassHierarchy::RemoveClass(UClass* InClass, TSet<FName>& InGameModules, FClassChanges& ClassChanges)
{
	FString OutClassPath;
	constexpr bool bIncludeClass = false;
	if (!GetClassPath(InClass, OutClassPath, InGameModules, bIncludeClass))
	{
		return;
	}
	
	// Most items aren't deep in the three but having a little buffer still help
	TArray<TSharedRef<FNativeClassHierarchyNode>, TInlineAllocator<10>> FoldersVisited;
	bool bHasProcessRootNodes = false;

	auto Visitor = [&FoldersVisited, &bHasProcessRootNodes, this](FStringView Token)
		{
			if (!bHasProcessRootNodes)
			{
				if (TSharedPtr<FNativeClassHierarchyNode>* NodePtr = RootNodes.Find(FName(Token)))
				{
					FoldersVisited.Add(NodePtr->ToSharedRef());
				}

				bHasProcessRootNodes = true;
			}
			else if (!FoldersVisited.IsEmpty())
			{
				if (TSharedPtr<FNativeClassHierarchyNode>* NodePtr = FoldersVisited.Last()->Children.Find(FNativeClassHierarchyNodeKey(FName(Token), ENativeClassHierarchyNodeType::Folder)))
				{
					FoldersVisited.Add(NodePtr->ToSharedRef());
				}
			}

		};

	UE::String::ParseTokens(FStringView(OutClassPath), TEXT('/'), TFunctionRef<void (FStringView)>(Visitor));

	if (!FoldersVisited.IsEmpty())
	{
		TSharedPtr<FNativeClassHierarchyNode> RemovedClass;
		bool bHasRemovedNode = false;

		bHasRemovedNode = FoldersVisited.Last()->Children.RemoveAndCopyValue(FNativeClassHierarchyNodeKey(InClass->GetFName(), ENativeClassHierarchyNodeType::Class), RemovedClass);
		ClassChanges.ClassesModified.Add(RemovedClass.ToSharedRef());

		if (bHasRemovedNode)
		{
			for (int Index = FoldersVisited.Num() - 2; Index >= 0; --Index)
			{
				const TSharedRef<FNativeClassHierarchyNode> ToRemoveIfEmpty = FoldersVisited[Index + 1];
				if (ToRemoveIfEmpty->Children.IsEmpty())
				{
					ClassChanges.FoldersModified.Add(ToRemoveIfEmpty);
					FoldersVisited[Index]->Children.Remove(FNativeClassHierarchyNodeKey(ToRemoveIfEmpty->EntryName, ENativeClassHierarchyNodeType::Folder));
				}
				else
				{
					bHasRemovedNode = false;
				}

				if (!bHasRemovedNode)
				{
					break;
				}
			}

			// If we removed all the node until the root. Check if we should remove the root also
			if (bHasRemovedNode)
			{
				const TSharedRef<FNativeClassHierarchyNode>& ToRemoveIfEmpty = FoldersVisited[0];
				if (ToRemoveIfEmpty->Children.IsEmpty())
				{
					ClassChanges.FoldersModified.Add(ToRemoveIfEmpty);
					RootNodes.Remove(ToRemoveIfEmpty->EntryName);
				}
			}
		}
	}
}

bool FNativeClassHierarchy::GetFileSystemPath(const FString& InClassPath, FString& OutFileSystemPath) const
{
	// Split the class path into its component parts
	TArray<FString> ClassPathParts;
	InClassPath.ParseIntoArray(ClassPathParts, TEXT("/"), true);
	
	// We need to have at least two sections (a root, and a module name) to be able to resolve a file system path
	if(ClassPathParts.Num() < 2)
	{
		return false;
	}

	// Is this path using a known root?
	TSharedPtr<FNativeClassHierarchyNode> RootNode = RootNodes.FindRef(*ClassPathParts[0]);
	if(!RootNode.IsValid())
	{
		return false;
	}

	// Is this path using a known module within that root?
	TSharedPtr<FNativeClassHierarchyNode> ModuleNode = RootNode->Children.FindRef(FNativeClassHierarchyNodeKey(*ClassPathParts[1], ENativeClassHierarchyNodeType::Folder));
	if(!ModuleNode.IsValid())
	{
		return false;
	}

	// Get the base file path to the module, and then append any remaining parts of the class path (as the remaining parts mirror the file system)
	if(FSourceCodeNavigation::FindModulePath(ClassPathParts[1], OutFileSystemPath))
	{
		for(int32 PathPartIndex = 2; PathPartIndex < ClassPathParts.Num(); ++PathPartIndex)
		{
			OutFileSystemPath /= ClassPathParts[PathPartIndex];
		}
		return true;
	}

	return false;
}

bool FNativeClassHierarchy::GetClassPath(const UClass* InClass, FString& OutClassPath, TSet<FName>& InGameModuleCache, const bool bIncludeClassName) const
{
	const FName ClassModuleName = GetClassModuleName(InClass);
	if(ClassModuleName.IsNone())
	{
		return false;
	}

	static const FName ModuleRelativePathMetaDataKey = "ModuleRelativePath";
	const FString& ClassModuleRelativeIncludePath = InClass->GetMetaData(ModuleRelativePathMetaDataKey);
	if(ClassModuleRelativeIncludePath.IsEmpty())
	{
		return false;
	}

	if (InGameModuleCache.Num() == 0)
	{
		InGameModuleCache = GetGameModules();
	}

	// Work out which root this class should go under
	TOptional<EPluginLoadedFrom> WhereLoadedFrom;
	const FName RootNodeName = GetClassPathRootForModule(ClassModuleName, InGameModuleCache, WhereLoadedFrom);

	// Work out the final path to this class within the hierarchy (which isn't the same as the path on disk)
	const FString ClassModuleRelativePath = ClassModuleRelativeIncludePath.Left(ClassModuleRelativeIncludePath.Find(TEXT("/"), ESearchCase::CaseSensitive, ESearchDir::FromEnd));
	OutClassPath = FString(TEXT("/")) + RootNodeName.ToString() + TEXT("/") + ClassModuleName.ToString();

	if(!ClassModuleRelativePath.IsEmpty())
	{
		OutClassPath += TEXT("/") + ClassModuleRelativePath;
	}

	if(bIncludeClassName)
	{
		OutClassPath += TEXT("/") + InClass->GetName();
	}

	return true;
}

FName FNativeClassHierarchy::GetFolderPathForModule(FName InModuleName, TSet<FName>& InGameModuleCache) const
{
	if (InGameModuleCache.Num() == 0)
	{
		InGameModuleCache = GetGameModules();
	}

	TOptional<EPluginLoadedFrom> WhereLoadedFrom;
	const FName RootNodeName = GetClassPathRootForModule(InModuleName, InGameModuleCache, WhereLoadedFrom);

	FNameBuilder FolderPath;
	FolderPath.AppendChar(TEXT('/'));
	RootNodeName.AppendString(FolderPath);
	FolderPath.AppendChar(TEXT('/'));
	InModuleName.AppendString(FolderPath);

	return FName(FolderPath);
}

void FNativeClassHierarchy::OnModulesChanged(FName InModuleName, EModuleChangeReason InModuleChangeReason)
{
	switch(InModuleChangeReason)
	{
	case EModuleChangeReason::ModuleLoaded:
		AddClassesForModule(InModuleName);
		break;

	case EModuleChangeReason::ModuleUnloaded:
		RemoveClassesForModule(InModuleName);
		break;

	default:
		break;
	}
}

void FNativeClassHierarchy::OnReloadReinstancingComplete()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNativeClassHierarchy::OnReloadReinstancingComplete);

	if (IReload* ActiveReload = GetActiveReloadInterface())
	{
		const TMap<UClass*, UClass*>& ReinstancedClasses = ActiveReload->GetReinstancedClasses();
		const TArray<UClass*>& NewClasses = ActiveReload->GetNewClasses();
		if (!ReinstancedClasses.IsEmpty() || !NewClasses.IsEmpty())
		{
			FClassChanges ClassChanges;

			TSet<FName> GameModules = GetGameModules();

			// Add new classes
			for (UClass* NewClass : NewClasses)
			{
				AddClass(NewClass, GameModules, ClassChanges);
			}

			// Replace re-instanced classes
			for (const TPair<UClass*, UClass*>& OldAndNew : ReinstancedClasses)
			{
				if (OldAndNew.Value)
				{
					// Adding the re-instanced class will replace old entry
					AddClass(OldAndNew.Value, GameModules, ClassChanges);
				}
			}

			if (!ClassChanges.FoldersModified.IsEmpty())
			{
				FoldersAddedDelegate.Broadcast(ClassChanges.FoldersModified);
			}
			if (!ClassChanges.ClassesModified.IsEmpty())
			{
				ClassesAddedDelegate.Broadcast(ClassChanges.ClassesModified);
			}

			// Reset the class changes tracking
			ClassChanges.Reset();

			// Then remove the old classes
			for (const TPair<UClass*, UClass*>& OldAndNew : ReinstancedClasses)
			{
				if (OldAndNew.Key && !OldAndNew.Value)
				{
					RemoveClass(OldAndNew.Key, GameModules, ClassChanges);
				}
			}

			if (!ClassChanges.ClassesModified.IsEmpty())
			{
				ClassesRemovedDelegate.Broadcast(ClassChanges.ClassesModified);
			}
			if (!ClassChanges.FoldersModified.IsEmpty())
			{
				FoldersRemovedDelegate.Broadcast(ClassChanges.FoldersModified);
			}
		}
	}
}

void FNativeClassHierarchy::OnReloadClassesAdded(const TArray<UClass*>& InAddedClasses)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNativeClassHierarchy::InAddedClasses);

	FClassChanges ClassChanges;
	TSet<FName> GameModules = GetGameModules();

	for (UClass* AddedClass : InAddedClasses)
	{
		AddClass(AddedClass, GameModules, ClassChanges);
	}

	if (!ClassChanges.FoldersModified.IsEmpty())
	{
		FoldersAddedDelegate.Broadcast(ClassChanges.FoldersModified);
	}
	if (!ClassChanges.ClassesModified.IsEmpty())
	{
		ClassesAddedDelegate.Broadcast(ClassChanges.ClassesModified);
	}
}

FName FNativeClassHierarchy::GetClassModuleName(const UClass* InClass)
{
	UPackage* const ClassPackage = InClass->GetOuterUPackage();

	if(ClassPackage)
	{
		return FPackageName::GetShortFName(ClassPackage->GetFName());
	}

	return NAME_None;
}

FName FNativeClassHierarchy::GetClassPathRootForModule(const FName& InModuleName, const TSet<FName>& InGameModules, TOptional<EPluginLoadedFrom>& OutWhereLoadedFrom)
{
	static const FName EngineRootNodeName = "Classes_Engine";
	static const FName GameRootNodeName = "Classes_Game";

	// Work out which root this class should go under (anything that isn't a game or plugin module goes under engine)
	FName RootNodeName = EngineRootNodeName;

	if(InGameModules.Contains(InModuleName))
	{
		RootNodeName = GameRootNodeName;
	}
	else
	{
		TSharedPtr<IPlugin> ModulePlugin = IPluginManager::Get().GetModuleOwnerPlugin(InModuleName);
		if (ModulePlugin.IsValid())
		{
			RootNodeName = FName(*(TEXT("Classes_") + ModulePlugin->GetName()));
			OutWhereLoadedFrom = ModulePlugin->GetLoadedFrom();
		}
	}

	return RootNodeName;
}

TSet<FName> FNativeClassHierarchy::GetGameModules()
{
	TSet<FName> GameModules;

	if (FApp::HasProjectName())
	{
		if (const FProjectDescriptor* const CurrentProject = IProjectManager::Get().GetCurrentProject())
		{
			for (const FModuleDescriptor& Module : CurrentProject->Modules)
			{
				GameModules.Add(Module.Name);
			}
		}
	}

	return GameModules;
}

void FNativeClassHierarchy::PopulateClassChanges(const TSharedPtr<FNativeClassHierarchyNode>& InNode, FClassChanges& OutClassChanges)
{
	for (const TPair<FNativeClassHierarchyNodeKey, TSharedPtr<FNativeClassHierarchyNode>>& Childs : InNode->Children)
	{
		PopulateClassChanges(Childs.Value, OutClassChanges);
	}

	if (InNode->Type == ENativeClassHierarchyNodeType::Folder)
	{
		OutClassChanges.FoldersModified.Add(InNode.ToSharedRef());
	}
	else if (InNode->Type == ENativeClassHierarchyNodeType::Class)
	{
		OutClassChanges.ClassesModified.Add(InNode.ToSharedRef());
	}
}
