// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "HAL/PlatformTime.h"

enum class EPluginLoadedFrom;

/**
 * Type of hierarchy node
 */
enum class ENativeClassHierarchyNodeType : uint8
{
	Folder,
	Class,
};

/**
 * Cache to avoid regenerating some necessary data during repeated calls to GetClassPathRootForModule during enumerate
 */
struct FNativeClassHierarchyGetClassPathCache
{
	TSet<FName> GameModules;

	void Reset()
	{
		GameModules.Reset();
	}
};

/**
 * Type used as a key in a map to resolve name conflicts between folders and classes
 */
struct FNativeClassHierarchyNodeKey
{
	FNativeClassHierarchyNodeKey(const FName InName, const ENativeClassHierarchyNodeType InType)
		: Name(InName)
		, Type(InType)
	{
	}

	FORCEINLINE bool operator==(const FNativeClassHierarchyNodeKey& Other) const
	{
		return Name == Other.Name && Type == Other.Type;
	}

	FORCEINLINE bool operator!=(const FNativeClassHierarchyNodeKey& Other) const
	{
		return Name != Other.Name || Type != Other.Type;
	}

	friend FORCEINLINE uint32 GetTypeHash(const FNativeClassHierarchyNodeKey& InKey)
	{
		uint32 KeyHash = 0;
		KeyHash = HashCombine(KeyHash, GetTypeHash(InKey.Name));
		KeyHash = HashCombine(KeyHash, GetTypeHash(InKey.Type));
		return KeyHash;
	}

	/** Name of this entry */
	FName Name;

	/** Type of this entry */
	ENativeClassHierarchyNodeType Type;
};

/**
 * Single node in the class hierarchy
 */
struct FNativeClassHierarchyNode
{
	/** Helper function to make a folder node entry */
	static TSharedRef<FNativeClassHierarchyNode> MakeFolderEntry(FName InEntryName, FString InEntryPath, TOptional<EPluginLoadedFrom> LoadedFrom);

	/** Helper function to make a class node entry */
	static TSharedRef<FNativeClassHierarchyNode> MakeClassEntry(UClass* InClass, FName InClassModuleName, FString InClassModuleRelativePath, FString InEntryPath, TOptional<EPluginLoadedFrom> LoadedFrom);

	void AddChild(TSharedRef<FNativeClassHierarchyNode> ChildEntry);

	/** Type of node, folder or class */
	ENativeClassHierarchyNodeType Type;

	/** The class this node is for (Type == Class) */
	UClass* Class;

	/** The name of the module the class is in (Type == Class) */
	FName ClassModuleName;

	/** Folder this class is in, relative to the class module (Type == Class) */
	FString ClassModuleRelativePath;

	/** Name used when showing this entry in the UI */
	FName EntryName;

	/** Path to this entry in the class hierarchy (not the same as the location on disk) */
	FString EntryPath;

	/** Child entries (Type == Folder) */
	TMap<FNativeClassHierarchyNodeKey, TSharedPtr<FNativeClassHierarchyNode>> Children;

	/** Which type of plugin this data was originally loaded from (if loaded from a plugin)*/
	TOptional<EPluginLoadedFrom> LoadedFrom;
};

/**
 * A filter used when querying the native class hierarchy.
 * Each component element is processed as an 'OR' operation while all the components are processed together as an 'AND' operation.
 */
struct FNativeClassHierarchyFilter
{
	/** The filter component for class paths */
	TArray<FName> ClassPaths;
	/** If true, ClassPaths components will be recursive */
	bool bRecursivePaths;

	FNativeClassHierarchyFilter()
	{
		bRecursivePaths = false;
	}

	/** Appends the other filter to this one */
	void Append(const FNativeClassHierarchyFilter& Other)
	{
		ClassPaths.Append(Other.ClassPaths);

		bRecursivePaths |= Other.bRecursivePaths;
	}

	/** Returns true if this filter has no entries */
	bool IsEmpty() const
	{
		return ClassPaths.Num() == 0;
	}

	/** Clears this filter of all entries */
	void Clear()
	{
		ClassPaths.Empty();

		bRecursivePaths = false;

		ensure(IsEmpty());
	}
};

/** 
 * Generates a hierarchical tree of native UObject classes based on their location in the file system as used by the asset view when showing C++ classes
 * This keeps its class hierarchy up-to-date as modules are loaded/unloaded
 */
class FNativeClassHierarchy
{
public:
	/** Constructor and destructor */
	FNativeClassHierarchy();
	~FNativeClassHierarchy();


	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNodesChanged, const TArrayView<TSharedRef<const FNativeClassHierarchyNode>>)

	/** Get the delegate called when folders are added to this class hierarchy */
	FOnNodesChanged& OnFoldersAdded()
	{
		return FoldersAddedDelegate;
	}

	/** Get the delegate called when classes are added to this class hierarchy */
	FOnNodesChanged& OnClassesAdded()
	{
		return ClassesAddedDelegate;
	}

	/** Get the delegate called when folders are removed to this class hierarchy */
	FOnNodesChanged& OnFoldersRemoved()
	{
		return FoldersRemovedDelegate;
	}

	/** Get the delegate called when classes are removed to this class hierarchy */
	FOnNodesChanged& OnClassesRemoved()
	{
		return ClassesRemovedDelegate;
	}

	/**
	 * Find the node for the given path, if any.
	 *
	 * @param InClassPath - The path to get the node of
	 * @param InType - The type of node to find
	 */
	TSharedPtr<const FNativeClassHierarchyNode> FindNode(const FName InClassPath, const ENativeClassHierarchyNodeType InType) const;

	/**
	 * Does the given path contain classes, optionally also testing sub-paths?
	 *
	 * @param InClassPath - The path to query classes in
	 * @param bRecursive - If true, the supplied path will be tested recursively
	 */
	bool HasClasses(const FName InClassPath, const bool bRecursive = false) const;

	/**
	 * Does the given path contain folders, optionally also testing sub-paths?
	 *
	 * @param InClassPath - The path to query classes in
	 * @param bRecursive - If true, the supplied path will be tested recursively
	 */
	bool HasFolders(const FName InClassPath, const bool bRecursive = false) const;

	/**
	 * Work out which classes known to the class hierarchy match the given filter
	 *
	 * @param Filter - The filter to apply when working out which classes match
	 * @param OutClasses - Array to be populated with matching classes
	 */
	void GetMatchingClasses(const FNativeClassHierarchyFilter& Filter, TArray<UClass*>& OutClasses) const;

	/**
	 * Work out which folders known to the class hierarchy match the given filter
	 *
	 * @param Filter - The filter to apply when working out which folders match
	 * @param OutFolders - Array to be populated with matching folder paths (these are in the form "/Classes_Name/ModuleName/SubFolder")
	 */
	void GetMatchingFolders(const FNativeClassHierarchyFilter& Filter, TArray<FString>& OutFolders) const;

	/**
	 * Get all root folders known to the class hierarchy
	 *
	 * @param OutClassRoots - Array to be populated with the known class path root folders (these are in the form "/Classes_Name")
	 * @param bIncludeEngineClasses - Whether we should include the "/Classes_Engine" root (and its associated folders) in the returned results
	 * @param bIncludePluginClasses - Whether we should include plugin class path roots (and their associated folders) in the returned results (these are all roots except for "/Classes_Game" and "/Classes_Engine")
	 */
	void GetClassRoots(TArray<FName>& OutClassRoots, const bool bIncludeEngineClasses = true, const bool bIncludePluginClasses = true) const;

	/**
	 * Get all folders known to the class hierarchy
	 *
	 * @param OutClassRoots - Array to be populated with the known class path root folders (these are in the form "/Classes_Name")
	 * @param OutClassFolders - Array to be populated with known class folder paths (these are in the form "/Classes_Name/ModuleName/SubFolder")
	 * @param bIncludeEngineClasses - Whether we should include the "/Classes_Engine" root (and its associated folders) in the returned results
	 * @param bIncludePluginClasses - Whether we should include plugin class path roots (and their associated folders) in the returned results (these are all roots except for "/Classes_Game" and "/Classes_Engine")
	 */
	void GetClassFolders(TArray<FName>& OutClassRoots, TArray<FString>& OutClassFolders, const bool bIncludeEngineClasses = true, const bool bIncludePluginClasses = true) const;

	/**
	 * Given a class path, work out the corresponding filesystem path on disk
	 * eg) Given "/Classes_Game/MyGame/MyAwesomeCode", we might resolve that to "../../../MyGame/Source/MyGame/MyAwesomeCode"
	 *
	 * @param InClassPath - The class path to try and resolve, eg) "/Classes_Game/MyGame/MyAwesomeCode"
	 * @param OutFileSystemPath - The string to fill with the resolved filesystem path, eg) "../../../MyGame/Source/MyGame/MyAwesomeCode"
	 * 
	 * @return true if the file system path could be resolved and OutFileSystemPath was filled in, false otherwise
	 */
	bool GetFileSystemPath(const FString& InClassPath, FString& OutFileSystemPath) const;

	/**
	 * Work out the class path that should be used for the given class
	 *
	 * @param InClass - The class path to try and get the class path for
	 * @param OutClassPath - The string to fill with the resolved class path, eg) "/Classes_Game/MyGame/MyAwesomeCode/MyAwesomeClass"
	 * @param bIncludeClassName - true to include the class name on the returned class path, false to get the class path of the containing folder
	 * 
	 * @return true if the class path could be resolved and OutClassPath was filled in, false otherwise
	 */
	bool GetClassPath(const UClass* InClass, FString& OutClassPath, TSet<FName>& InGameModuleCache, const bool bIncludeClassName = true) const;

	/**
	 * Work out the folder path that should be used for the given module
	 *
	 * @param InModuleName - The module we want to find the folder path for
	 * @param InGameModuleCache - The list of modules that belong to the "/Classes_Game" root
	 * 
	 * @return The folder path for the module
	 */
	FName GetFolderPathForModule(FName InModuleName, TSet<FName>& InGameModuleCache) const;

	/**
	 * Test if root node passes given rules
	 */
	bool RootNodePassesFilter(const FName InRootName, const TSharedPtr<const FNativeClassHierarchyNode>& InRootNode, const bool bIncludeEngineClasses, const bool bIncludePluginClasses) const;

	/**
	 * Test if root node passes given rules
	 */
	bool RootNodePassesFilter(const FName InRootName, const bool bIncludeEngineClasses, const bool bIncludePluginClasses) const;

private:
	struct FClassChanges
	{
		FClassChanges()
		{
		}

		void Reset()
		{
			ClassesModified.Reset();
			FoldersModified.Reset();
		}

		TArray<TSharedRef<const FNativeClassHierarchyNode>> ClassesModified;
		TArray<TSharedRef<const FNativeClassHierarchyNode>> FoldersModified;
	};

	/**
	 * Test to see whether the given node has any child classes
	 *
	 * @param HierarchyNode - The node to test the children of
	 * @param bRecurse - True to recurse into sub-folders, false to only check the given node
	 *
	 * @return True if the node has child classes, false otherwise
	 */
	static bool HasClassesRecursive(const TSharedRef<FNativeClassHierarchyNode>& HierarchyNode, const bool bRecurse = true);

	/**
	 * Test to see whether the given node has any child folders
	 *
	 * @param HierarchyNode - The node to test the children of
	 * @param bRecurse - True to recurse into sub-folders, false to only check the given node
	 *
	 * @return True if the node has child folders, false otherwise
	 */
	static bool HasFoldersRecursive(const TSharedRef<FNativeClassHierarchyNode>& HierarchyNode, const bool bRecurse = true);

	/**
	 * Update OutClasses with any classes that are children of the given node
	 *
	 * @param HierarchyNode - The node to add the children of
	 * @param OutClasses - Array to be populated with the child classes
	 * @param bRecurse - True to recurse into sub-folders, false to only check the given node
	 */
	static void GetClassesRecursive(const TSharedRef<FNativeClassHierarchyNode>& HierarchyNode, TArray<UClass*>& OutClasses, const bool bRecurse = true);

	/**
	 * Update OutFolders with any folders that are children of the given node
	 *
	 * @param HierarchyNode - The node to add the children of
	 * @param OutFolders - Array to be populated with the child folders
	 * @param bRecurse - True to recurse into sub-folders, false to only check the given node
	 */
	static void GetFoldersRecursive(const TSharedRef<FNativeClassHierarchyNode>& HierarchyNode, TArray<FString>& OutFolders, const bool bRecurse = true);

	/**
	 * Populate OutMatchingNodes with the nodes that correspond to the given class paths
	 *
	 * @param InClassPaths - The class paths to find the nodes for, or an empty list to get all root nodes
	 * @param OutMatchingNodes - Array to be populated the nodes that correspond to the given class paths
	 * @param InType - Type of node to search for, ie: class / folder
	 */
	void GatherMatchingNodesForPaths(const TArrayView<const FName>& InClassPaths, TArray<TSharedRef<FNativeClassHierarchyNode>, TInlineAllocator<4>>& OutMatchingNodes, const ENativeClassHierarchyNodeType InType = ENativeClassHierarchyNodeType::Folder) const;

	/**
	 * Completely clear and re-populate the known class hierarchy
	 */
	void PopulateHierarchy();

	/**
	 * Append any classes from the given module to the known class hierarchy
	 *
	 * @param InModuleName - The name of the module to add
	 */
	void AddClassesForModule(const FName& InModuleName);

	/**
	 * Remove any classes in the given module from the known class hierarchy
	 *
	 * @param InModuleName - The name of the module to remove
	 */
	void RemoveClassesForModule(const FName& InModuleName);

	/**
	 * Add a single class to the known class hierarchy, creating any folders as required
	 *
	 * @param InClass - The class that is to be added
	 * @param InGameModules - The list of modules that belong to the "/Classes_Game" root
	 * @param FClassChanges - Track the new classes and folders that were added.
	 */
	void AddClass(UClass* InClass, const TSet<FName>& InGameModules, FClassChanges& ClassChanges);

	/**
	 * Remove a single class to the known class hierarchy, removing any empty folders left
	 *
	 * @param InClass - The class that is to be removed
	 * @param InGameModules - The list of modules that belong to the "/Classes_Game" root
	 * @param FClassChanges - Track the classes and folders that were removed.
	 */
	void RemoveClass(UClass* InClass, TSet<FName>& InGameModules, FClassChanges& ClassChanges);

	/**
	 * Called when we're notified that a module has changed status
	 *
	 * @param InModuleName - The module that changed status
	 * @param InModuleChangeReason - Why the module changed status
	 */
	void OnModulesChanged(FName InModuleName, EModuleChangeReason InModuleChangeReason);

	/**
	 * Called when we're notified that a reloaded module has reinstanced some classes.
	 */
	void OnReloadReinstancingComplete();

	/**
	 * Called when we're notified that a reloaded module has some new classes.
	 */
	 void OnReloadClassesAdded(const TArray<UClass*>& InAddedClasses);

	/**
	 * Given a class, work out which module it belongs to
	 *
	 * @param InClass - The class we want to find the module for
	 *
	 * @return The name of the module that holds the class, eg) "CoreUObject"
	 */
	static FName GetClassModuleName(const UClass* InClass);

	/**
	 * Given a module, work out which root path it should use as a parent
	 *
	 * @param InModuleName - The module we want to find the root path for
	 * @param InGameModules - The list of modules that belong to the "/Classes_Game" root
	 * @param WhereLoadedFrom - Determine where this class was loaded from.
	 * 
	 * @return The name of the root path to use, eg "/Classes_Game"
	 */
	static FName GetClassPathRootForModule(const FName& InModuleName, const TSet<FName>& InGameModules, TOptional<EPluginLoadedFrom>& OutWhereLoadedFrom);

	/**
	 * Calculate the list of modules that belong to the "/Classes_Game" root
	 */
	static TSet<FName> GetGameModules();

	void PopulateClassChanges(const TSharedPtr<FNativeClassHierarchyNode>& InNode, FClassChanges& OutClassChanges);

	/** Root level nodes corresponding to the root folders used by the Content Browser, eg) Classes_Engine, Classes_Game, etc */
	TMap<FName, TSharedPtr<FNativeClassHierarchyNode>> RootNodes;

	/** Delegate called when the class hierarchy as added some new folders */
	FOnNodesChanged FoldersAddedDelegate;

	/** Delegate called when the class hierarchy as added some new classes */
	FOnNodesChanged ClassesAddedDelegate;

	/** Delegate called when the class hierarchy as removed some folders */
	FOnNodesChanged FoldersRemovedDelegate;

	/** Delegate called when the class hierarchy as removed some classes */
	FOnNodesChanged ClassesRemovedDelegate;
};
