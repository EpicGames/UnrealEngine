// Copyright Epic Games, Inc. All Rights Reserved.

#include "NewClassContextMenu.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "ContentBrowserClassDataSource"

void FNewClassContextMenu::MakeContextMenu(
	UToolMenu* Menu, 
	const TArray<FName>& InSelectedClassPaths,
	const FOnNewClassRequested& InOnNewClassRequested
	)
{
	if (InSelectedClassPaths.Num() == 0)
	{
		return;
	}

	const FName FirstSelectedPath = InSelectedClassPaths[0];
	const bool bHasSinglePathSelected = InSelectedClassPaths.Num() == 1;

	auto CanExecuteClassActions = [bHasSinglePathSelected]() -> bool
	{
		// We can execute class actions when we only have a single path selected
		return bHasSinglePathSelected;
	};
	const FCanExecuteAction CanExecuteClassActionsDelegate = FCanExecuteAction::CreateLambda(CanExecuteClassActions);

	// Add Class
	if(InOnNewClassRequested.IsBound())
	{
		FName ClassCreationPath = FirstSelectedPath;
		FText NewClassToolTip;
		if(bHasSinglePathSelected)
		{
			NewClassToolTip = FText::Format(LOCTEXT("NewClassTooltip_CreateIn", "Create a new class in {0}."), FText::FromName(ClassCreationPath));
		}
		else
		{
			NewClassToolTip = LOCTEXT("NewClassTooltip_InvalidNumberOfPaths", "Can only create classes when there is a single path selected.");
		}

		{
			FToolMenuSection& Section = Menu->AddSection("ContentBrowserNewClass", LOCTEXT("ClassMenuHeading", "C++ Class"));
			Section.AddMenuEntry(
				"NewClass",
				LOCTEXT("NewClassLabel", "New C++ Class..."),
				NewClassToolTip,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.AddCodeToProject"),
				FUIAction(
					FExecuteAction::CreateStatic(&FNewClassContextMenu::ExecuteNewClass, ClassCreationPath, InOnNewClassRequested),
					CanExecuteClassActionsDelegate
					)
				);
		}
	}
}

void FNewClassContextMenu::ExecuteNewClass(FName InPath, FOnNewClassRequested InOnNewClassRequested)
{
	// An empty path override will cause the class wizard to use the default project path
	InOnNewClassRequested.ExecuteIfBound(InPath);
}

#undef LOCTEXT_NAMESPACE
