// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class UToolMenu;

class FNewClassContextMenu
{
public:
	DECLARE_DELEGATE_OneParam(FOnNewClassRequested, const FName /*SelectedPath*/);

	/** Makes the context menu widget */
	static void MakeContextMenu(
		UToolMenu* Menu, 
		const TArray<FName>& InSelectedClassPaths,
		const FOnNewClassRequested& InOnNewClassRequested
		);

private:
	/** Create a new class at the specified path */
	static void ExecuteNewClass(FName InPath, FOnNewClassRequested InOnNewClassRequested);
};
