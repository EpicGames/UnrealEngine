// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserItemData.h"

class FContentBrowserClassFileItemDataPayload;
class FContentBrowserClassFolderItemDataPayload;

class IAssetTypeActions;
class FAssetThumbnail;
class UContentBrowserDataSource;

namespace ContentBrowserClassData
{

	CONTENTBROWSERCLASSDATASOURCE_API bool IsEngineClass(const FName InPath);

	CONTENTBROWSERCLASSDATASOURCE_API bool IsProjectClass(const FName InPath);

	CONTENTBROWSERCLASSDATASOURCE_API bool IsPluginClass(const FName InPath);

	CONTENTBROWSERCLASSDATASOURCE_API FContentBrowserItemData CreateClassFolderItem(UContentBrowserDataSource* InOwnerDataSource, const FName InVirtualPath, const FName InFolderPath, const bool bIsFromPlugin);

	CONTENTBROWSERCLASSDATASOURCE_API FContentBrowserItemData CreateClassFileItem(UContentBrowserDataSource* InOwnerDataSource, const FName InVirtualPath, const FName InClassPath, UClass* InClass, const bool bIsFromPlugin);

	CONTENTBROWSERCLASSDATASOURCE_API TSharedPtr<const FContentBrowserClassFolderItemDataPayload> GetClassFolderItemPayload(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem);

	CONTENTBROWSERCLASSDATASOURCE_API TSharedPtr<const FContentBrowserClassFileItemDataPayload> GetClassFileItemPayload(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem);
	
	CONTENTBROWSERCLASSDATASOURCE_API void EnumerateClassFolderItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserClassFolderItemDataPayload>&)> InFolderPayloadCallback);

	CONTENTBROWSERCLASSDATASOURCE_API void EnumerateClassFileItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserClassFileItemDataPayload>&)> InClassPayloadCallback);

	CONTENTBROWSERCLASSDATASOURCE_API void EnumerateClassItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserClassFolderItemDataPayload>&)> InFolderPayloadCallback, TFunctionRef<bool(const TSharedRef<const FContentBrowserClassFileItemDataPayload>&)> InClassPayloadCallback);

	CONTENTBROWSERCLASSDATASOURCE_API void SetOptionalErrorMessage(FText* OutErrorMsg, FText InErrorMsg);

	CONTENTBROWSERCLASSDATASOURCE_API bool CanEditItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	CONTENTBROWSERCLASSDATASOURCE_API bool CanEditClassFileItem(const FContentBrowserClassFileItemDataPayload& InClassPayload, FText* OutErrorMsg);

	CONTENTBROWSERCLASSDATASOURCE_API bool EditItems(IAssetTypeActions* InClassTypeActions, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems);

	CONTENTBROWSERCLASSDATASOURCE_API bool EditClassFileItems(IAssetTypeActions* InClassTypeActions, TArrayView<const TSharedRef<const FContentBrowserClassFileItemDataPayload>> InClassPayloads);

	CONTENTBROWSERCLASSDATASOURCE_API bool UpdateItemThumbnail(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FAssetThumbnail& InThumbnail);

	CONTENTBROWSERCLASSDATASOURCE_API bool UpdateClassFileItemThumbnail(const FContentBrowserClassFileItemDataPayload& InClassPayload, FAssetThumbnail& InThumbnail);

	CONTENTBROWSERCLASSDATASOURCE_API bool AppendItemReference(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& InOutStr);

	CONTENTBROWSERCLASSDATASOURCE_API bool AppendItemObjectPath(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& InOutStr);

	CONTENTBROWSERCLASSDATASOURCE_API bool AppendItemPackageName(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& InOutStr);

	CONTENTBROWSERCLASSDATASOURCE_API bool AppendClassFileItemReference(const FContentBrowserClassFileItemDataPayload& InClassPayload, FString& InOutStr);

	CONTENTBROWSERCLASSDATASOURCE_API bool AppendObjectPathFileItemReference(const FContentBrowserClassFileItemDataPayload& InClassPayload, FString& InOutStr);

	CONTENTBROWSERCLASSDATASOURCE_API bool AppendPackageNameItemReference(const FContentBrowserClassFileItemDataPayload& InClassPayload, FString& InOutStr);

	CONTENTBROWSERCLASSDATASOURCE_API bool GetItemPhysicalPath(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& OutDiskPath);

	CONTENTBROWSERCLASSDATASOURCE_API bool GetClassFolderItemPhysicalPath(const FContentBrowserClassFolderItemDataPayload& InFolderPayload, FString& OutDiskPath);

	CONTENTBROWSERCLASSDATASOURCE_API bool GetClassFileItemPhysicalPath(const FContentBrowserClassFileItemDataPayload& InClassPayload, FString& OutDiskPath);

	CONTENTBROWSERCLASSDATASOURCE_API bool GetItemAttribute(IAssetTypeActions* InClassTypeActions, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue);

	CONTENTBROWSERCLASSDATASOURCE_API bool GetClassFolderItemAttribute(const FContentBrowserClassFolderItemDataPayload& InFolderPayload, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue);

	CONTENTBROWSERCLASSDATASOURCE_API bool GetClassFileItemAttribute(IAssetTypeActions* InClassTypeActions, const FContentBrowserClassFileItemDataPayload& InClassPayload, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue);

	CONTENTBROWSERCLASSDATASOURCE_API bool GetItemAttributes(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues);

	CONTENTBROWSERCLASSDATASOURCE_API bool GetClassFileItemAttributes(const FContentBrowserClassFileItemDataPayload& InClassPayload, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues);

}
