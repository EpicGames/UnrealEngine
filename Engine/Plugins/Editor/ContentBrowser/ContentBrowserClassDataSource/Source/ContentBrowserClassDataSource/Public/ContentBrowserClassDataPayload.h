// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserItemData.h"
#include "AssetRegistry/AssetData.h"

#define UE_API CONTENTBROWSERCLASSDATASOURCE_API

class FAssetThumbnail;

class FContentBrowserClassFolderItemDataPayload : public IContentBrowserItemDataPayload
{
public:
	explicit FContentBrowserClassFolderItemDataPayload(const FName InInternalPath)
		: InternalPath(InInternalPath)
	{
	}

	FName GetInternalPath() const
	{
		return InternalPath;
	}

	UE_API const FString& GetFilename() const;

private:
	FName InternalPath;

	mutable bool bHasCachedFilename = false;
	mutable FString CachedFilename;
};

class FContentBrowserClassFileItemDataPayload : public IContentBrowserItemDataPayload
{
public:
	UE_API FContentBrowserClassFileItemDataPayload(const FName InInternalPath, UClass* InClass);

	FName GetInternalPath() const
	{
		return InternalPath;
	}

	UClass* GetClass() const
	{
		return Class.Get();
	}

	const FAssetData& GetAssetData() const
	{
		return AssetData;
	}

	UE_API const FString& GetFilename() const;

	UE_API void UpdateThumbnail(FAssetThumbnail& InThumbnail) const;

private:
	FName InternalPath;

	TWeakObjectPtr<UClass> Class;

	FAssetData AssetData;

	mutable bool bHasCachedFilename = false;
	mutable FString CachedFilename;
};

#undef UE_API
