// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserDataSource.h"
#include "NativeClassHierarchy.h"
#include "UObject/Package.h"
#include "ContentBrowserClassDataSource.generated.h"

#define UE_API CONTENTBROWSERCLASSDATASOURCE_API

class FContentBrowserClassFileItemDataPayload;
class FContentBrowserClassFolderItemDataPayload;

class IAssetTypeActions;
class UToolMenu;
class FNativeClassHierarchy;
struct FCollectionNameType;

USTRUCT()
struct FContentBrowserCompiledClassDataFilter
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TSet<TObjectPtr<UClass>> ValidClasses;

	UPROPERTY()
	TSet<FName> ValidFolders;
};

UCLASS(MinimalAPI)
class UContentBrowserClassDataSource : public UContentBrowserDataSource
{
	GENERATED_BODY()

public:
	UE_API void Initialize(const bool InAutoRegister = true);

	UE_API virtual void Shutdown() override;

	UE_API virtual void CompileFilter(const FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter) override;

	UE_API virtual void EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) override;

	UE_API virtual void EnumerateItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) override;

	UE_API virtual void EnumerateItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) override;

	UE_API virtual bool EnumerateItemsForObjects(const TArrayView<UObject*> InObjects, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) override;

	UE_API virtual bool IsFolderVisible(const FName InPath, const EContentBrowserIsFolderVisibleFlags InFlags, const FContentBrowserFolderContentsFilter& InContentsFilter) override;

	UE_API virtual bool DoesItemPassFilter(const FContentBrowserItemData& InItem, const FContentBrowserDataCompiledFilter& InFilter) override;

	UE_API virtual bool GetItemAttribute(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue) override;

	UE_API virtual bool GetItemAttributes(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues) override;

	UE_API virtual bool GetItemPhysicalPath(const FContentBrowserItemData& InItem, FString& OutDiskPath) override;

	UE_API virtual bool CanEditItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg) override;

	UE_API virtual bool EditItem(const FContentBrowserItemData& InItem) override;

	UE_API virtual bool BulkEditItems(TArrayView<const FContentBrowserItemData> InItems) override;

	UE_API virtual bool AppendItemReference(const FContentBrowserItemData& InItem, FString& InOutStr) override;

	UE_API virtual bool AppendItemObjectPath(const FContentBrowserItemData& InItem, FString& InOutStr) override;

	UE_API virtual bool AppendItemPackageName(const FContentBrowserItemData& InItem, FString& InOutStr) override;

	UE_API virtual bool UpdateThumbnail(const FContentBrowserItemData& InItem, FAssetThumbnail& InThumbnail) override;

	UE_API virtual bool TryGetCollectionId(const FContentBrowserItemData& InItem, FSoftObjectPath& OutCollectionId) override;

	UE_API virtual bool Legacy_TryGetPackagePath(const FContentBrowserItemData& InItem, FName& OutPackagePath) override;

	UE_API virtual bool Legacy_TryGetAssetData(const FContentBrowserItemData& InItem, FAssetData& OutAssetData) override;

	UE_API virtual bool Legacy_TryConvertPackagePathToVirtualPath(const FName InPackagePath, FName& OutPath) override;

	UE_API virtual bool Legacy_TryConvertAssetDataToVirtualPath(const FAssetData& InAssetData, const bool InUseFolderPaths, FName& OutPath) override;

protected:
	UE_API TSharedPtr<IAssetTypeActions> GetClassTypeActions();
	
	UE_API virtual void BuildRootPathVirtualTree() override;
	UE_API bool RootClassPathPassesFilter(const FName InRootClassPath, const bool bIncludeEngineClasses, const bool bIncludePluginClasses) const;

private:
	UE_API bool IsKnownClassPath(const FName InPackagePath) const;

	UE_API bool GetClassPathsForCollections(TArrayView<const FCollectionRef> InCollections, const bool bIncludeChildCollections, TArray<FTopLevelAssetPath>& OutClassPaths);

	UE_API FContentBrowserItemData CreateClassFolderItem(const FName InFolderPath);

	UE_API FContentBrowserItemData CreateClassFileItem(UClass* InClass, FNativeClassHierarchyGetClassPathCache& InCache);

	UE_API FContentBrowserItemData CreateClassFolderItem(const FName InFolderPath, const TSharedPtr<const FNativeClassHierarchyNode>& InFolderNode);

	UE_API FContentBrowserItemData CreateClassFileItem(const FName InClassPath, const TSharedPtr<const FNativeClassHierarchyNode>& InClassNode);

	UE_API TSharedPtr<const FContentBrowserClassFolderItemDataPayload> GetClassFolderItemPayload(const FContentBrowserItemData& InItem) const;

	UE_API TSharedPtr<const FContentBrowserClassFileItemDataPayload> GetClassFileItemPayload(const FContentBrowserItemData& InItem) const;

	UE_API void OnNewClassRequested(const FName InSelectedPath);

	UE_API void PopulateAddNewContextMenu(UToolMenu* InMenu);

	UE_API void ConditionalCreateNativeClassHierarchy();

	UE_API void OnFoldersAdded(const TArrayView<TSharedRef<const FNativeClassHierarchyNode>> InFolders);

	UE_API void OnFoldersRemoved(const TArrayView<TSharedRef<const FNativeClassHierarchyNode>> InFolders);

	UE_API void OnClassesAdded(const TArrayView<TSharedRef<const FNativeClassHierarchyNode>> InClasses);

	UE_API void OnClassesRemoved(const TArrayView<TSharedRef<const FNativeClassHierarchyNode>> InClasses);

	TSharedPtr<FNativeClassHierarchy> NativeClassHierarchy;
	FNativeClassHierarchyGetClassPathCache NativeClassHierarchyGetClassPathCache;

	TSharedPtr<IAssetTypeActions> ClassTypeActions;
};

#undef UE_API
